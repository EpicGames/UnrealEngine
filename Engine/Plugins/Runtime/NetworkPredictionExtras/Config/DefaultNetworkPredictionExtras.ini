[CoreRedirects]
+ClassRedirects=(OldName="/Script/NetworkPredictionExtras.NetworkPhysicsComponent", NewName="/Script/NetworkPrediction.NetworkPhysicsComponent")
