{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Network Prediction Extras",
	"Description": "Non essential classes for Network Prediction. Samples, test maps, etc intended to help developers start using the system. Not intended to be used directly in a shipping product.",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NetworkPredictionExtras",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NetworkPredictionExtrasLatentLoad",
			"Type": "Runtime",
			"LoadingPhase": "None"
		}
	],
	"Plugins": [
		{
			"Name": "NetworkPrediction",
			"Enabled": true
		}
	]
}