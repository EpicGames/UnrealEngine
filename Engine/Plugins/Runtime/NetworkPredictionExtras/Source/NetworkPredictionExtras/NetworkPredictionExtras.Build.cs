// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetworkPredictionExtras : ModuleRules
	{
		public NetworkPredictionExtras(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"NetworkPrediction",
					"Core",
                    "CoreUObject",
                    "Engine",
                    "RenderCore",
					"InputCore",
					"PhysicsCore",
					"Chaos",
					"Engine"
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"NetCore",
 					"TraceLog",
					"Engine"
                }
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
				}
				);

			// Only needed for the PIE delegate in FNetworkPredictionModule::StartupModule
            if (Target.Type == TargetType.Editor) {
                PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                });
            }

		}
	}
}
