// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseMovementComponent.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"

// ----------------------------------------------------------------------------------------------------------
//	UBaseMovementComponent setup/init
// ----------------------------------------------------------------------------------------------------------

UBaseMovementComponent::UBaseMovementComponent()
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;
	
	bWantsInitializeComponent = true;
	bAutoActivate = true;
	SetIsReplicatedByDefault(true);
	bWantsInitializeComponent = true;
}

void UBaseMovementComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	// Super may start up the tick function when we don't want to.
	UpdateTickRegistration();

	// If the owner ticks, make sure we tick first. This is to ensure the owner's location will be up to date when it ticks.
	AActor* Owner = GetOwner();
	
	if (bRegister && PrimaryComponentTick.bCanEverTick && Owner && Owner->CanEverTick())
	{
		Owner->PrimaryActorTick.AddPrerequisite(this, PrimaryComponentTick);
	}
}

void UBaseMovementComponent::UpdateTickRegistration()
{
	const bool bHasUpdatedComponent = (UpdatedComponent != NULL);
	SetComponentTickEnabled(bHasUpdatedComponent && bAutoActivate);
}

void UBaseMovementComponent::InitializeComponent()
{
	TGuardValue<bool> InInitializeComponentGuard(bInInitializeComponent, true);

	// RootComponent is null in OnRegister for blueprint (non-native) root components.
	if (!UpdatedComponent)
	{
		// Auto-register owner's root component if found.
		if (AActor* MyActor = GetOwner())
		{
			if (USceneComponent* NewUpdatedComponent = MyActor->GetRootComponent())
			{
				SetUpdatedComponent(NewUpdatedComponent);
			}
			else
			{
				ensureMsgf(false, TEXT("No root component found on %s. Simulation initialization will most likely fail."), *GetPathNameSafe(MyActor));
			}
		}
	}
	
	Super::InitializeComponent();
}

void UBaseMovementComponent::OnRegister()
{
	TGuardValue<bool> InOnRegisterGuard(bInOnRegister, true);

	UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	Super::OnRegister();

	const UWorld* MyWorld = GetWorld();
	if (MyWorld && MyWorld->IsGameWorld())
	{
		USceneComponent* NewUpdatedComponent = UpdatedComponent;
		if (!UpdatedComponent)
		{
			// Auto-register owner's root component if found.
			AActor* MyActor = GetOwner();
			if (MyActor)
			{
				NewUpdatedComponent = MyActor->GetRootComponent();
			}
		}

		SetUpdatedComponent(NewUpdatedComponent);
	}
}

void UBaseMovementComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	if (UpdatedComponent && UpdatedComponent != NewUpdatedComponent)
	{
		UpdatedComponent->SetShouldUpdatePhysicsVolume(false);
		if (IsValid(UpdatedComponent))
		{
			UpdatedComponent->SetPhysicsVolume(NULL, true);
			UpdatedComponent->PhysicsVolumeChangedDelegate.RemoveDynamic(this, &UBaseMovementComponent::PhysicsVolumeChanged);
		}

		// remove from tick prerequisite
		UpdatedComponent->PrimaryComponentTick.RemovePrerequisite(this, PrimaryComponentTick); 
	}

	if (UpdatedPrimitive && UpdatedPrimitive != NewUpdatedComponent)
	{
		UpdatedPrimitive->OnComponentBeginOverlap.RemoveDynamic(this, &UBaseMovementComponent::OnBeginOverlap);
	}

	// Don't assign pending kill components, but allow those to null out previous UpdatedComponent.
	UpdatedComponent = GetValid(NewUpdatedComponent);
	UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);

	// Assign delegates
	if (UpdatedComponent)
	{
		UpdatedComponent->SetShouldUpdatePhysicsVolume(true);
		UpdatedComponent->PhysicsVolumeChangedDelegate.AddUniqueDynamic(this, &UBaseMovementComponent::PhysicsVolumeChanged);

		if (!bInOnRegister && !bInInitializeComponent)
		{
			// UpdateOverlaps() in component registration will take care of this.
			UpdatedComponent->UpdatePhysicsVolume(true);
		}
		
		// force ticks after movement component updates
		UpdatedComponent->PrimaryComponentTick.AddPrerequisite(this, PrimaryComponentTick); 
	}

	if (IsValid(UpdatedPrimitive))
	{
		UpdatedPrimitive->OnComponentBeginOverlap.AddDynamic(this, &UBaseMovementComponent::OnBeginOverlap);
	}


	UpdateTickRegistration();
}

void UBaseMovementComponent::PhysicsVolumeChanged(APhysicsVolume* NewVolume)
{
	// This itself feels bad. When will this be called? Its impossible to know what is allowed and not allowed to be done in this callback.
	// Callbacks instead should be trapped within the simulation update function. This isn't really possible though since the UpdateComponent
	// is the one that will call this.
}

