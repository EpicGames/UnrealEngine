// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseMovementSimulation.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"

namespace BaseMovementCVars
{
	static float PenetrationPullbackDistance = 0.125f;
	static FAutoConsoleVariableRef CVarPenetrationPullbackDistance(TEXT("bm.PenetrationPullbackDistance"),
		PenetrationPullbackDistance,
		TEXT("Pull out from penetration of an object by this extra distance.\n")
		TEXT("Distance added to penetration fix-ups."),
		ECVF_Default);

	static float PenetrationOverlapCheckInflation = 0.100f;
	static FAutoConsoleVariableRef CVarPenetrationOverlapCheckInflation(TEXT("bm.PenetrationOverlapCheckInflation"),
		PenetrationOverlapCheckInflation,
		TEXT("Inflation added to object when checking if a location is free of blocking collision.\n")
		TEXT("Distance added to inflation in penetration overlap check."),
		ECVF_Default);

	static int32 RequestMispredict = 0;
	static FAutoConsoleVariableRef CVarRequestMispredict(TEXT("bm.RequestMispredict"),
		RequestMispredict, TEXT("Causes a misprediction by inserting random value into stream on authority side"), ECVF_Default);
}

DEFINE_LOG_CATEGORY_STATIC(LogBaseMovement, Log, All);

// ----------------------------------------------------------------------------------------------------------
//	Movement System Driver
//
//	NOTE: Most of the Movement Driver is not ideal! We are at the mercy of the UpdateComponent since it is the
//	the object that owns its collision data and its MoveComponent function. Ideally we would have everything within
//	the movement simulation code and it do its own collision queries. But instead we have to come back to the Driver/Component
//	layer to do this kind of stuff.
//
// ----------------------------------------------------------------------------------------------------------

FVector FBaseMovementSimulation::GetPenetrationAdjustment(const FHitResult& Hit) const
{
	if (!Hit.bStartPenetrating)
	{
		return FVector::ZeroVector;
	}

	FVector Result;
	const float PullBackDistance = FMath::Abs(BaseMovementCVars::PenetrationPullbackDistance);
	const float PenetrationDepth = (Hit.PenetrationDepth > 0.f ? Hit.PenetrationDepth : 0.125f);

	Result = Hit.Normal * (PenetrationDepth + PullBackDistance);

	return Result;
}

bool FBaseMovementSimulation::OverlapTest(const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor) const
{
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(MovementOverlapTest), false, IgnoreActor);
	FCollisionResponseParams ResponseParam;
	InitCollisionParams(QueryParams, ResponseParam);
	return UpdatedComponent->GetWorld()->OverlapBlockingTestByChannel(Location, RotationQuat, CollisionChannel, CollisionShape, QueryParams, ResponseParam);
}

void FBaseMovementSimulation::InitCollisionParams(FCollisionQueryParams &OutParams, FCollisionResponseParams& OutResponseParam) const
{
	if (UpdatedPrimitive)
	{
		UpdatedPrimitive->InitSweepCollisionParams(OutParams, OutResponseParam);
	}
}

bool FBaseMovementSimulation::ResolvePenetration(const FVector& ProposedAdjustment, const FHitResult& Hit, const FQuat& NewRotationQuat) const
{
	// SceneComponent can't be in penetration, so this function really only applies to PrimitiveComponent.
	const FVector Adjustment = ProposedAdjustment; //ConstrainDirectionToPlane(ProposedAdjustment);
	if (!Adjustment.IsZero() && UpdatedPrimitive)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BaseMovementComponent_ResolvePenetration);
		// See if we can fit at the adjusted location without overlapping anything.
		AActor* ActorOwner = UpdatedComponent->GetOwner();
		if (!ActorOwner)
		{
			return false;
		}

		UE_LOGF(LogBaseMovement, Verbose, "ResolvePenetration: %ls.%ls at location %ls inside %ls.%ls at location %ls by %.3f (netmode: %d)",
			*ActorOwner->GetName(),
			*UpdatedComponent->GetName(),
			*UpdatedComponent->GetComponentLocation().ToString(),
			*Hit.GetHitObjectHandle().GetName(),
			*GetNameSafe(Hit.GetComponent()),
			Hit.Component.IsValid() ? *Hit.GetComponent()->GetComponentLocation().ToString() : TEXT("<unknown>"),
			Hit.PenetrationDepth,
			(uint32)ActorOwner->GetNetMode());

		// We really want to make sure that precision differences or differences between the overlap test and sweep tests don't put us into another overlap,
		// so make the overlap test a bit more restrictive.
		const float OverlapInflation = BaseMovementCVars::PenetrationOverlapCheckInflation;
		bool bEncroached = OverlapTest(Hit.TraceStart + Adjustment, NewRotationQuat, UpdatedPrimitive->GetCollisionObjectType(), UpdatedPrimitive->GetCollisionShape(OverlapInflation), ActorOwner);
		if (!bEncroached)
		{
			// Move without sweeping.
			MoveUpdatedComponent(Adjustment, NewRotationQuat, false, nullptr, ETeleportType::TeleportPhysics);
			UE_LOGF(LogBaseMovement, Verbose, "ResolvePenetration:   teleport by %ls", *Adjustment.ToString());
			return true;
		}
		else
		{
			// Disable MOVECOMP_NeverIgnoreBlockingOverlaps if it is enabled, otherwise we wouldn't be able to sweep out of the object to fix the penetration.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, EMoveComponentFlags(MoveComponentFlags & (~MOVECOMP_NeverIgnoreBlockingOverlaps)));

			// Try sweeping as far as possible...
			FHitResult SweepOutHit(1.f);
			bool bMoved = MoveUpdatedComponent(Adjustment, NewRotationQuat, true, &SweepOutHit, ETeleportType::TeleportPhysics);
			UE_LOGF(LogBaseMovement, Verbose, "ResolvePenetration:   sweep by %ls (success = %d)", *Adjustment.ToString(), bMoved);

			// Still stuck?
			if (!bMoved && SweepOutHit.bStartPenetrating)
			{
				// Combine two MTD results to get a new direction that gets out of multiple surfaces.
				const FVector SecondMTD = GetPenetrationAdjustment(SweepOutHit);
				const FVector CombinedMTD = Adjustment + SecondMTD;
				if (SecondMTD != Adjustment && !CombinedMTD.IsZero())
				{
					bMoved = MoveUpdatedComponent(CombinedMTD, NewRotationQuat, true, nullptr, ETeleportType::TeleportPhysics);
					UE_LOGF(LogBaseMovement, Verbose, "ResolvePenetration:   sweep by %ls (MTD combo success = %d)", *CombinedMTD.ToString(), bMoved);
				}
			}

			// Still stuck?
			if (!bMoved)
			{
				// Try moving the proposed adjustment plus the attempted move direction. This can sometimes get out of penetrations with multiple objects
				const FVector MoveDelta = (Hit.TraceEnd - Hit.TraceStart); //ConstrainDirectionToPlane(Hit.TraceEnd - Hit.TraceStart);
				if (!MoveDelta.IsZero())
				{
					bMoved = MoveUpdatedComponent(Adjustment + MoveDelta, NewRotationQuat, true, nullptr, ETeleportType::TeleportPhysics);
					UE_LOGF(LogBaseMovement, Verbose, "ResolvePenetration:   sweep by %ls (adjusted attempt success = %d)", *(Adjustment + MoveDelta).ToString(), bMoved);
				}
			}	

			return bMoved;
		}
	}

	return false;
}

bool FBaseMovementSimulation::SafeMoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport) const
{
	if (UpdatedComponent == NULL)
	{
		OutHit.Reset(1.f);
		return false;
	}

	bool bMoveResult = false;

	// Scope for move flags
	{
		bMoveResult = MoveUpdatedComponent(Delta, NewRotation, bSweep, &OutHit, Teleport);
	}

	// Handle initial penetrations
	if (OutHit.bStartPenetrating && UpdatedComponent)
	{
		const FVector RequestedAdjustment = GetPenetrationAdjustment(OutHit);
		if (ResolvePenetration(RequestedAdjustment, OutHit, NewRotation))
		{
			// Retry original move
			bMoveResult = MoveUpdatedComponent(Delta, NewRotation, bSweep, &OutHit, Teleport);
		}
	}

	return bMoveResult;
}

bool FBaseMovementSimulation::MoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit, ETeleportType Teleport) const
{
	if (UpdatedComponent)
	{
		const FVector NewDelta = Delta;
		return UpdatedComponent->MoveComponent(NewDelta, NewRotation, bSweep, OutHit, MoveComponentFlags, Teleport);
	}

	return false;
}


FTransform FBaseMovementSimulation::GetUpdateComponentTransform() const
{
	if (ensure(UpdatedComponent))
	{
		return UpdatedComponent->GetComponentTransform();		
	}
	return FTransform::Identity;
}
