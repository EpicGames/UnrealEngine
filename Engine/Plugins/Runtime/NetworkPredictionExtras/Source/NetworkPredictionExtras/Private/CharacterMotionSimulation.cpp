// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterMotionSimulation.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "NetworkPredictionTickState.h"

DEFINE_LOG_CATEGORY_STATIC(LogCharacterMotionSimulation, Log, All);

namespace CharacterMotionSimCVars
{
static float ErrorTolerance = 10.f;
static FAutoConsoleVariableRef CVarErrorTolerance(TEXT("CharacterMotion.ErrorTolerance"),
	ErrorTolerance, TEXT("Location tolerance for reconcile"), ECVF_Default);
}

// -------------------------------------------------------------------------------------------------------

bool FCharacterMotionAuxState::ShouldReconcile(const FCharacterMotionAuxState& AuthorityState) const
{
	return false;
}

bool FCharacterMotionSyncState::ShouldReconcile(const FCharacterMotionSyncState& AuthorityState) const
{
	const float ErrorTolerance = CharacterMotionSimCVars::ErrorTolerance;
	return !AuthorityState.Location.Equals(Location, ErrorTolerance);
}

// -------------------------------------------------------------------------------------------------------

bool FCharacterMotionSimulation::ForceMispredict = false;
static FVector CMForceMispredictVelocityMagnitude = FVector(2000.f, 0.f, 0.f);

void FFloorTestResult::SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor)
{
	bBlockingHit = InHit.IsValidBlockingHit();
	bWalkableFloor = bIsWalkableFloor;
	FloorDist = InSweepFloorDist;
	HitResult = InHit;
}

bool FCharacterMotionSimulation::IsExceedingMaxSpeed(const FVector& Velocity, float InMaxSpeed) const
{
	InMaxSpeed = FMath::Max(0.f, InMaxSpeed);
	const float MaxSpeedSquared = FMath::Square(InMaxSpeed);
	
	// Allow 1% error tolerance, to account for numeric imprecision.
	const float OverVelocityPercent = 1.01f;
	return (Velocity.SizeSquared() > MaxSpeedSquared * OverVelocityPercent);
}

FVector FCharacterMotionSimulation::ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const
{
	return FVector::VectorPlaneProject(Delta, Normal) * Time;
}

void FCharacterMotionSimulation::TwoWallAdjust(FVector& OutDelta, const FHitResult& Hit, const FVector& OldHitNormal) const
{
	FVector Delta = OutDelta;
	const FVector HitNormal = Hit.Normal;

	if ((OldHitNormal | HitNormal) <= 0.f) //90 or less corner, so use cross product for direction
	{
		const FVector DesiredDir = Delta;
		FVector NewDir = (HitNormal ^ OldHitNormal);
		NewDir = NewDir.GetSafeNormal();
		Delta = (Delta | NewDir) * (1.f - Hit.Time) * NewDir;
		if ((DesiredDir | Delta) < 0.f)
		{
			Delta = -1.f * Delta;
		}
	}
	else //adjust to new wall
	{
		const FVector DesiredDir = Delta;
		Delta = ComputeSlideVector(Delta, 1.f - Hit.Time, HitNormal, Hit);
		if ((Delta | DesiredDir) <= 0.f)
		{
			Delta = FVector::ZeroVector;
		}
		else if ( FMath::Abs((HitNormal | OldHitNormal) - 1.f) < KINDA_SMALL_NUMBER )
		{
			// we hit the same wall again even after adjusting to move along it the first time
			// nudge away from it (this can happen due to precision issues)
			Delta += HitNormal * 0.01f;
		}
	}

	OutDelta = Delta;
}

float FCharacterMotionSimulation::SlideAlongSurface(const FVector& Delta, float Time, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	float PercentTimeApplied = 0.f;
	const FVector OldHitNormal = Normal;

	FVector SlideDelta = ComputeSlideVector(Delta, Time, Normal, Hit);

	if ((SlideDelta | Delta) > 0.f)
	{
		SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit, ETeleportType::None);

		const float FirstHitPercent = Hit.Time;
		PercentTimeApplied = FirstHitPercent;
		if (Hit.IsValidBlockingHit())
		{
			// Notify first impact
			if (bHandleImpact)
			{
				// !HandleImpact(Hit, FirstHitPercent * Time, SlideDelta);
			}

			// Compute new slide normal when hitting multiple surfaces.
			TwoWallAdjust(SlideDelta, Hit, OldHitNormal);

			// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
			if (!SlideDelta.IsNearlyZero(1e-3f) && (SlideDelta | Delta) > 0.f)
			{
				// Perform second move
				SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit, ETeleportType::None);
				const float SecondHitPercent = Hit.Time * (1.f - FirstHitPercent);
				PercentTimeApplied += SecondHitPercent;

				// Notify second impact
				if (bHandleImpact && Hit.bBlockingHit)
				{
					// !HandleImpact(Hit, SecondHitPercent * Time, SlideDelta);
				}
			}
		}

		return FMath::Clamp(PercentTimeApplied, 0.f, 1.f);
	}

	return 0.f;
}


void FCharacterMotionSimulation::InvalidateCache()
{
	CachedFloor.Clear();
}


void FCharacterMotionSimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output)
{
	InvalidateCache();
	*Output.Sync = *Input.Sync;

	const float DeltaSeconds = (float)TimeStep.StepMS / 1000.f;

	// --------------------------------------------------------------
	//	Rotation Update
	//	We do the rotational update inside the movement sim so that things like server side teleport will work.
	//	(We want rotation to be treated the same as location, with respect to how its updated, corrected, etc).
	//	In this simulation, the rotation update isn't allowed to "fail". We don't expect the collision query to be able to fail the rotational update.
	// --------------------------------------------------------------

	const FRotator LocalSpaceRotation = ComputeLocalRotation(TimeStep, Input, Output);

	Output.Sync->Rotation += (LocalSpaceRotation * DeltaSeconds);
	Output.Sync->Rotation.Normalize();

	const FVector LocalSpaceMovementInput = ComputeLocalInput(TimeStep, Input, Output);
	   	
	// ===================================================

	// --------------------------------------------------------------
	//	Calculate the final movement delta and move the update component
	// --------------------------------------------------------------
	
	PerformMovement(DeltaSeconds, Input, Output, LocalSpaceMovementInput);

	// Finalize. This is unfortunate. The component mirrors our internal motion state and since we call into it to update, at this point, it has the real position.
	const FTransform UpdateComponentTransform = GetUpdateComponentTransform();
	Output.Sync->Location = UpdateComponentTransform.GetLocation();

	// Note that we don't pull the rotation out of the final update transform. Converting back from a quat will lead to a different FRotator than what we are storing
	// here in the simulation layer. This may not be the best choice for all movement simulations, but is ok for this one.
}


FRotator FCharacterMotionSimulation::ComputeLocalRotation(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const
{
	FRotator LocalRotation = Input.Cmd->RotationInput;

	// Only move character around Yaw axis, for now.
	LocalRotation.Pitch = 0.f;
	LocalRotation.Roll = 0.f;

	return LocalRotation;
}


FVector FCharacterMotionSimulation::ComputeLocalInput(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const
{
	FVector LocalInput = Output.Sync->Rotation.RotateVector(Input.Cmd->MovementInput);

	// Adjust per movement mode
	switch (Input.Sync->MovementMode)
	{
	case ECharacterMovementMode::Walking:
	{
		LocalInput.Z = 0.f;
		// Re-normalize in 2D XY space so you don't walk slower when looking down or up.
		LocalInput = LocalInput.GetSafeNormal();
		break;
	}

	case ECharacterMovementMode::Falling:
	{
		LocalInput.Z = 0.f;
		break;
	}

	default:
		break;
	}

	return LocalInput;
}


FVector FCharacterMotionSimulation::ComputeVelocity(float DeltaSeconds, const FVector& InitialVelocity, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput) const
{
	const FVector ControlAcceleration = LocalSpaceMovementInput.GetClampedToMaxSize(1.f);
	FVector Velocity = InitialVelocity;

	const float AnalogInputModifier = (ControlAcceleration.SizeSquared() > 0.f ? ControlAcceleration.Size() : 0.f);
	const float MaxPawnSpeed = Input.Aux->MaxSpeed * AnalogInputModifier;
	const bool bExceedingMaxSpeed = IsExceedingMaxSpeed(Velocity, MaxPawnSpeed);

	if (AnalogInputModifier > 0.f && !bExceedingMaxSpeed)
	{
		// Apply change in velocity direction
		if (Velocity.SizeSquared() > 0.f)
		{
			// TODO: Friction from UCharacterMovementComponent
			// Change direction faster than only using acceleration, but never increase velocity magnitude.
			const float TimeScale = FMath::Clamp(DeltaSeconds * Input.Aux->TurningBoost, 0.f, 1.f);
			Velocity = Velocity + (ControlAcceleration * Velocity.Size() - Velocity) * TimeScale;
		}
	}
	else
	{
		// Dampen velocity magnitude based on deceleration.
		if (Velocity.SizeSquared() > 0.f)
		{
			// TODO: Friction from UCharacterMovementComponent
			const FVector OldVelocity = Velocity;
			const float VelSize = FMath::Max(Velocity.Size() - FMath::Abs(Input.Aux->Deceleration) * DeltaSeconds, 0.f);
			Velocity = Velocity.GetSafeNormal() * VelSize;

			// Don't allow braking to lower us below max speed if we started above it.
			if (bExceedingMaxSpeed && Velocity.SizeSquared() < FMath::Square(MaxPawnSpeed))
			{
				Velocity = OldVelocity.GetSafeNormal() * MaxPawnSpeed;
			}
		}
	}

	// Apply acceleration and clamp velocity magnitude.
	const float NewMaxSpeed = (IsExceedingMaxSpeed(Velocity, MaxPawnSpeed)) ? Velocity.Size() : MaxPawnSpeed;
	Velocity += ControlAcceleration * FMath::Abs(Input.Aux->Acceleration) * DeltaSeconds;
	Velocity = Velocity.GetClampedToMaxSize(NewMaxSpeed);

	if (FCharacterMotionSimulation::ForceMispredict)
	{
		Velocity += CMForceMispredictVelocityMagnitude;
		FCharacterMotionSimulation::ForceMispredict = false;
	}

	return Velocity;
}


void FCharacterMotionSimulation::PerformMovement(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput)
{
	// TODO: Correctly integrate movement step using midpoint integration when applying velocity:
	//       0.5 * (OldVelocity + NewVelocity) * dt

	// TODO: each movement mode function should be able to only partially consume the time, and return with more time left to simulate.
	// This allows us to handle changes from walking to falling halfway through the move for example.

	switch (Input.Sync->MovementMode)
	{
	case ECharacterMovementMode::None:
		break;

	case ECharacterMovementMode::Walking:
		Movement_Walking(DeltaSeconds, Input, Output, LocalSpaceMovementInput);
		break;

	case ECharacterMovementMode::Falling:
		Movement_Falling(DeltaSeconds, Input, Output, LocalSpaceMovementInput);
		break;

	default:
		break;
	}

	// Update velocity
	// We don't want position changes to vastly reverse our direction (which can happen due to penetration fixups etc)
	// TODO: this isn't correct with midpoint integration. But we should have a deflected velocity.
	/*
	if (!bPositionCorrected)
	{
		const FVector NewLocation = UpdatedComponent->GetComponentLocation();
		Velocity = ((NewLocation - OldLocation) / DeltaTime);
	}*/
}


void FCharacterMotionSimulation::Movement_Walking(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput)
{
	Output.Sync->Velocity = ComputeVelocity(DeltaSeconds, Input.Sync->Velocity, Input, Output, LocalSpaceMovementInput);
	Output.Sync->Velocity.Z = 0.f;

	FVector Delta = Output.Sync->Velocity * DeltaSeconds;
	const FQuat OutputQuat = Output.Sync->Rotation.Quaternion();
	FHitResult Hit(1.f);

	if (!Delta.IsNearlyZero(1e-6f))
	{
		SafeMoveUpdatedComponent(Delta, OutputQuat, true, Hit, ETeleportType::None);
	}

	// Hits could invalidate this, though that really sucks to check every time. Should probably avoid allowing this during the simulation so we don't have to keep checking it every action
	// which is error-prone and easy to overlook.
	if (!UpdatedPrimitive)
	{
		return;
	}

	FindFloor(Input, Output, UpdatedPrimitive->GetComponentLocation(), CachedFloor);
	if (!CachedFloor.IsWalkableFloor())
	{
		SetMovementMode(ECharacterMovementMode::Falling, Input, Output);
		// TODO: allow more sim time.
		return;
	}

	if (Hit.IsValidBlockingHit())
	{
		// Try to slide the remaining distance along the surface.
		SlideAlongSurface(Delta, 1.f - Hit.Time, OutputQuat, Hit.Normal, Hit, true);
	}

	if (!UpdatedPrimitive) 
	{
		return;
	}

	FindFloor(Input, Output, UpdatedPrimitive->GetComponentLocation(), CachedFloor);
	if (!CachedFloor.IsWalkableFloor())
	{
		SetMovementMode(ECharacterMovementMode::Falling, Input, Output);
		// TODO: allow more sim time.
		return;
	}
}


void FCharacterMotionSimulation::Movement_Falling(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput)
{
	const FVector InputVelocity = Input.Sync->Velocity;
	
	// We don't want velocity limits to account for the Z velocity, since it is limited by terminal velocity of the environment.
	const FVector HorizontalVelocity = FVector(InputVelocity.X, InputVelocity.Y, 0.f);
	Output.Sync->Velocity = ComputeVelocity(DeltaSeconds, HorizontalVelocity, Input, Output, LocalSpaceMovementInput);
	Output.Sync->Velocity.Z = InputVelocity.Z;

	FVector OldVelocity = Output.Sync->Velocity;
	FVector Delta = OldVelocity * DeltaSeconds;
	const FQuat OutputQuat = Output.Sync->Rotation.Quaternion();

	const FVector Gravity = ComputeGravity(DeltaSeconds, Input, Output, Delta);
	const FVector FallVelocity = OldVelocity + (Gravity * DeltaSeconds);

	// Midpoint integration
	FVector FallDelta = 0.5f * (OldVelocity + FallVelocity) * DeltaSeconds;

	FHitResult Hit(1.f);
	SafeMoveUpdatedComponent(FallDelta, OutputQuat, true, Hit, ETeleportType::None);
	
	// Compute final velocity based on how long we actually go until we get a hit.
	Output.Sync->Velocity.Z = 0.5f * (OldVelocity.Z + (FallVelocity.Z * Hit.Time));

	// Handle impact
	if (Hit.IsValidBlockingHit() && UpdatedPrimitive)
	{
		if (IsValidLandingSpot(UpdatedPrimitive->GetComponentLocation(), Hit, Input, Output, FallDelta))
		{
			SetMovementMode(ECharacterMovementMode::Walking, Input, Output);
			// TODO: Return substep for next move.
			return;
		}

		float LastMoveTimeSlice = DeltaSeconds;
		float SubTimeTickRemaining = LastMoveTimeSlice * (1.f - Hit.Time);

		const FVector OldHitNormal = Hit.Normal;
		const FVector DeflectDelta = ComputeSlideVector(FallDelta, 1.f - Hit.Time, OldHitNormal, Hit);

		if (SubTimeTickRemaining > KINDA_SMALL_NUMBER && (DeflectDelta | FallDelta) > 0.f)
		{
			// Move in deflected direction.
			SafeMoveUpdatedComponent(DeflectDelta, OutputQuat, true, Hit, ETeleportType::None);

			if (Hit.IsValidBlockingHit() && UpdatedPrimitive)
			{
				if (IsValidLandingSpot(UpdatedPrimitive->GetComponentLocation(), Hit, Input, Output, DeflectDelta))
				{
					SetMovementMode(ECharacterMovementMode::Walking, Input, Output);
					// TODO: Return substep for next move.
					return;
				}
			}
		}
	}
}


void FCharacterMotionSimulation::SetMovementMode(ECharacterMovementMode MovementMode, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output)
{
	Output.Sync->MovementMode = MovementMode;

	// TODO: notifications, restrictions, etc.
	if (MovementMode == ECharacterMovementMode::Walking)
	{
		Output.Sync->Velocity.Z = 0.f;
	}
}


FVector FCharacterMotionSimulation::ComputeGravity(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Delta) const
{
	//
	// FIXME: temporary. Should support arbitrary direction, and pull gravity value from the environment.
	//

	return FVector(0.f, 0.f, -9800.f);
}


bool FCharacterMotionSimulation::IsWalkable(const FHitResult& Hit, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const
{
	//
	// FIXME: temporary. Should support arbitrary direction.
	//

	if (!Hit.IsValidBlockingHit())
	{
		// No hit, or starting in penetration
		return false;
	}

	// Never walk up vertical surfaces.
	if (Hit.ImpactNormal.Z < KINDA_SMALL_NUMBER)
	{
		return false;
	}

	float TestWalkableZ = Input.Aux->WalkableFloorZ;

	// See if this component overrides the walkable floor z.
	const UPrimitiveComponent* HitComponent = Hit.Component.Get();
	if (HitComponent)
	{
		const FWalkableSlopeOverride& SlopeOverride = HitComponent->GetWalkableSlopeOverride();
		TestWalkableZ = SlopeOverride.ModifyWalkableFloorZ(TestWalkableZ);
	}

	// Can't walk on this surface if it is too steep.
	if (Hit.ImpactNormal.Z < TestWalkableZ)
	{
		return false;
	}

	return true;
}

bool FCharacterMotionSimulation::IsValidLandingSpot(const FVector& Location, const FHitResult& Hit, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Delta) const
{
	//
	// FIXME: temporary
	//

	if (!Hit.bBlockingHit)
	{
		return false;
	}

	if (Hit.bStartPenetrating)
	{
		return false;
	}

	// Reject unwalkable floor normals.
	if (!IsWalkable(Hit, Input, Output))
	{
		return false;
	}

	// Make sure floor test passes here.
	FFloorTestResult FloorResult;
	FindFloor(Input, Output, Location, FloorResult);

	if (!FloorResult.IsWalkableFloor())
	{
		return false;
	}

	return true;
}


void FCharacterMotionSimulation::FindFloor(const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Location, FFloorTestResult& OutFloorResult) const
{
	if (!UpdatedComponent || !UpdatedComponent->IsQueryCollisionEnabled())
	{
		OutFloorResult.Clear();
		return;
	}

	// Sweep for the floor
	const float SweepDistance = Input.Aux->FloorSweepDistance;
	ComputeFloorDist(Input, Output, Location, OutFloorResult, SweepDistance);
}


void FCharacterMotionSimulation::ComputeFloorDist(const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Location, FFloorTestResult& OutFloorResult, float SweepDistance) const
{
	OutFloorResult.Clear();

	// Sweep test
	if (SweepDistance > 0.f && UpdatedPrimitive)
	{
		FHitResult Hit(1.f);
		FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ComputeFloorDist), false, UpdatedPrimitive->GetOwner());
		FCollisionResponseParams ResponseParam;
		InitCollisionParams(QueryParams, ResponseParam);
		const ECollisionChannel CollisionChannel = UpdatedComponent->GetCollisionObjectType();

		// Use a shorter height to avoid sweeps giving weird results if we start on a surface.
		// This also allows us to adjust out of penetrations.
		// TODO: pluggable shapes
		float PawnRadius, PawnHalfHeight;
		UpdatedPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);

		const float ShrinkScale = 0.9f;
		const float ShrinkScaleOverlap = 0.1f;
		float ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScale);
		float TraceDist = SweepDistance + ShrinkHeight;
		FCollisionShape CapsuleShape = FCollisionShape::MakeCapsule(PawnRadius, PawnHalfHeight - ShrinkHeight);

		// TODO: arbitrary direction
		const FVector SweepDirection = FVector(0.f, 0.f, -SweepDistance);
		bool bBlockingHit = FloorSweepTest(Hit, Location, Location + SweepDirection, CollisionChannel, CapsuleShape, QueryParams, ResponseParam);

		if (bBlockingHit)
		{
			/*
			// Reject hits adjacent to us, we only care about hits on the bottom portion of our capsule.
			// Check 2D distance to impact point, reject if within a tolerance from radius.
			if (Hit.bStartPenetrating) // || !IsWithinEdgeTolerance(Location, Hit.ImpactPoint, CapsuleShape.Capsule.Radius))
			{
				// Use a capsule with a slightly smaller radius and shorter height to avoid the adjacent object.
				// Capsule must not be nearly zero or the trace will fall back to a line trace from the start point and have the wrong length.
				CapsuleShape.Capsule.Radius = FMath::Max(0.f, CapsuleShape.Capsule.Radius - SWEEP_EDGE_REJECT_DISTANCE - KINDA_SMALL_NUMBER);
				if (!CapsuleShape.IsNearlyZero())
				{
					ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScaleOverlap);
					TraceDist = SweepDistance + ShrinkHeight;
					CapsuleShape.Capsule.HalfHeight = FMath::Max(PawnHalfHeight - ShrinkHeight, CapsuleShape.Capsule.Radius);
					Hit.Reset(1.f, false);

					bBlockingHit = FloorSweepTest(Hit, CapsuleLocation, CapsuleLocation + FVector(0.f, 0.f, -TraceDist), CollisionChannel, CapsuleShape, QueryParams, ResponseParam);
				}
			}
			*/

			// Reduce hit distance by ShrinkHeight because we shrank the capsule for the trace.
			// We allow negative distances here, because this allows us to pull out of penetrations.
			const float MAX_FLOOR_DIST = 1.5f;
			const float MaxPenetrationAdjust = FMath::Max(MAX_FLOOR_DIST, PawnRadius);
			const float SweepResult = FMath::Max(-MaxPenetrationAdjust, Hit.Time * TraceDist - ShrinkHeight);

			OutFloorResult.SetFromSweep(Hit, SweepResult, false);
			if (Hit.IsValidBlockingHit() && IsWalkable(Hit, Input, Output))
			{
				if (SweepResult <= SweepDistance)
				{
					// Hit within test distance.
					OutFloorResult.bWalkableFloor = true;
				}
			}
		}
	}
}


bool FCharacterMotionSimulation::FloorSweepTest(
	FHitResult& OutHit,
	const FVector& Start,
	const FVector& End,
	ECollisionChannel TraceChannel,
	const struct FCollisionShape& CollisionShape,
	const struct FCollisionQueryParams& Params,
	const struct FCollisionResponseParams& ResponseParam
) const
{
	bool bBlockingHit = false;
	
	if (UpdatedPrimitive)
	{
		bBlockingHit = UpdatedPrimitive->GetWorld()->SweepSingleByChannel(OutHit, Start, End, FQuat::Identity, TraceChannel, CollisionShape, Params, ResponseParam);
	}

	return bBlockingHit;
}


void FCharacterMotionSimulation::OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult)
{	

}
