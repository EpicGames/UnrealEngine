// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlyingMovementComponent.h"
#include "FlyingMovementSimulation.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "NetworkPredictionProxyWrite.h"

#include "GameFramework/CharacterMovementComponent.h" // To facilitate A/B testing by disabling CMC if present
#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

DEFINE_LOG_CATEGORY_STATIC(LogFlyingMovement, Log, All);

namespace FlyingMovementCVars
{
static float MaxSpeed = 1200.f;
static FAutoConsoleVariableRef CVarMaxSpeed(TEXT("motion.MaxSpeed"),
	MaxSpeed,
	TEXT("Temp value for testing changes to max speed."),
	ECVF_Default);

static int32 RequestMispredict = 0;
static FAutoConsoleVariableRef CVarRequestMispredict(TEXT("fp.RequestMispredict"),
	RequestMispredict, TEXT("Causes a misprediction by inserting random value into stream on authority side"), ECVF_Default);
}

float UFlyingMovementComponent::GetDefaultMaxSpeed() { return FlyingMovementCVars::MaxSpeed; }

// ----------------------------------------------------------------------------------------------------------
//	FFlyingMovementModelDef: the piece that ties everything together that we use to register with the NP system.
// ----------------------------------------------------------------------------------------------------------

class FFlyingMovementModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = FFlyingMovementSimulation;
	using StateTypes = FlyingMovementStateTypes;
	using Driver = UFlyingMovementComponent;

	static const TCHAR* GetName() { return TEXT("FlyingMovement"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers; }
};

NP_MODEL_REGISTER(FFlyingMovementModelDef);

// ----------------------------------------------------------------------------------------------------------
//	UFlyingMovementComponent
// ----------------------------------------------------------------------------------------------------------

UFlyingMovementComponent::UFlyingMovementComponent()
{

}

void UFlyingMovementComponent::InitializeNetworkPredictionProxy()
{
	OwnedMovementSimulation = MakePimpl<FFlyingMovementSimulation>();
	InitFlyingMovementSimulation(OwnedMovementSimulation.Get());
	
	NetworkPredictionProxy.Init<FFlyingMovementModelDef>(GetWorld(), GetReplicationProxies(), OwnedMovementSimulation.Get(), this);
}

void UFlyingMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	// TEMP! Disable existing CMC if it is activate. Just makes A/B testing easier for now.
	if (AActor* Owner = GetOwner())
	{
		if (UCharacterMovementComponent* OldComp = Owner->FindComponentByClass<UCharacterMovementComponent>())
		{
			if (OldComp->IsActive())
			{
				OldComp->Deactivate();
			}
		}

		Owner->SetReplicatingMovement(false);
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	const ENetRole OwnerRole = GetOwnerRole();

	// Check if we should trip a mispredict. (Note how its not possible to do this inside the Update function!)
	if (OwnerRole == ROLE_Authority && FlyingMovementCVars::RequestMispredict)
	{
		FFlyingMovementSimulation::ForceMispredict = true;
		FlyingMovementCVars::RequestMispredict = 0;
	}


	static bool bDrawSimLocation = true;
	static bool bDrawPresentationLocation = true;
	/*
	if (bDrawSimLocation)
	{
		DrawDebugSphere(GetWorld(), 
		NetworkPredictionProxy.ReadSyncState<FFlyingMovementSyncState>(ENetworkPredictionStateRead::Simulation)->Location + FVector(0.f, 0.f, 100.f),
		25.f, 12, FColor::Green, false, -1.f);
	}
	if (bDrawPresentationLocation)
	{
		DrawDebugSphere(GetWorld(), 
			NetworkPredictionProxy.ReadSyncState<FFlyingMovementSyncState>()->Location + FVector(0.f, 0.f, 200.f),
			25.f, 12, FColor::Red, false, -1.f);
	}
	*/
	// Temp
	/*
	if (OwnerRole == ROLE_Authority)
	{
		if (MovementAuxState->Get()->MaxSpeed != FlyingMovementCVars::MaxSpeed)
		{
			MovementAuxState->Modify([](FFlyingMovementAuxState& Aux)
			{
				Aux.MaxSpeed = FlyingMovementCVars::MaxSpeed;
			});
		}
	}
	*/
}

void UFlyingMovementComponent::OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult)
{
	if (ActiveMovementSimulation)
	{
		ActiveMovementSimulation->OnBeginOverlap(OverlappedComp, Other, OtherComp, OtherBodyIndex, bFromSweep, SweepResult);
	}
}

void UFlyingMovementComponent::ProduceInput(const int32 DeltaTimeMS, FFlyingMovementInputCmd* Cmd)
{
	// This isn't ideal. It probably makes sense for the component to do all the input binding rather.
	ProduceInputDelegate.ExecuteIfBound(DeltaTimeMS, *Cmd);
}

void UFlyingMovementComponent::RestoreFrame(const FFlyingMovementSyncState* SyncState, const FFlyingMovementAuxState* AuxState)
{
	FTransform Transform(SyncState->Rotation.Quaternion(), SyncState->Location, UpdatedComponent->GetComponentTransform().GetScale3D() );
	UpdatedComponent->SetWorldTransform(Transform, false, nullptr, ETeleportType::TeleportPhysics);
	UpdatedComponent->ComponentVelocity = SyncState->Velocity;
}

void UFlyingMovementComponent::FinalizeFrame(const FFlyingMovementSyncState* SyncState, const FFlyingMovementAuxState* AuxState)
{
	// The component will often be in the "right place" already on FinalizeFrame, so a comparison check makes sense before setting it.
	if (UpdatedComponent->GetComponentLocation().Equals(SyncState->Location) == false || UpdatedComponent->GetComponentQuat().Rotator().Equals(SyncState->Rotation, FFlyingMovementSimulation::ROTATOR_TOLERANCE) == false)
	{
		RestoreFrame(SyncState, AuxState);
	}
}

void UFlyingMovementComponent::InitializeSimulationState(FFlyingMovementSyncState* Sync, FFlyingMovementAuxState* Aux)
{
	npCheckSlow(UpdatedComponent);
	npCheckSlow(Sync);
	npCheckSlow(Aux);

	Sync->Location = UpdatedComponent->GetComponentLocation();
	Sync->Rotation = UpdatedComponent->GetComponentQuat().Rotator();

	Aux->MaxSpeed = GetDefaultMaxSpeed();
}

// Init function. This is broken up from ::InstantiateNetworkedSimulation and templated so that subclasses can share the init code
void UFlyingMovementComponent::InitFlyingMovementSimulation(FFlyingMovementSimulation* Simulation)
{
	check(UpdatedComponent);
	check(ActiveMovementSimulation == nullptr); // Reinstantiation not supported
	ActiveMovementSimulation = Simulation;

	Simulation->SetComponents(UpdatedComponent, UpdatedPrimitive);
}

float UFlyingMovementComponent::GetMaxMoveSpeed() const
{
	if (const FFlyingMovementAuxState* AuxState = NetworkPredictionProxy.ReadAuxState<FFlyingMovementAuxState>())
	{
		return AuxState->MaxSpeed;
	}
	return 0;
}

void UFlyingMovementComponent::SetMaxMoveSpeed(float NewMaxMoveSpeed)
{
	NetworkPredictionProxy.WriteAuxState<FFlyingMovementAuxState>([NewMaxMoveSpeed](FFlyingMovementAuxState& AuxState)
	{
		AuxState.MaxSpeed = NewMaxMoveSpeed;
	}, "SetMaxMoveSpeed");
}

void UFlyingMovementComponent::AddMaxMoveSpeed(float AdditiveMaxMoveSpeed)
{
	NetworkPredictionProxy.WriteAuxState<FFlyingMovementAuxState>([AdditiveMaxMoveSpeed](FFlyingMovementAuxState& AuxState)
	{
		AuxState.MaxSpeed += AdditiveMaxMoveSpeed;
	}, "AddMaxMoveSpeed");
}
