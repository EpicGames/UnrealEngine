// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlyingMovementSimulation.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionTrace.h"

DEFINE_LOG_CATEGORY_STATIC(LogFlyingPawnSimulation, Log, All);

namespace FlyingPawnSimCVars
{
static float ErrorTolerance = 10.f;
static FAutoConsoleVariableRef CVarErrorTolerance(TEXT("fp.ErrorTolerance"),
	ErrorTolerance, TEXT("Location tolerance for reconcile"), ECVF_Default);
}

// -------------------------------------------------------------------------------------------------------

bool FFlyingMovementAuxState::ShouldReconcile(const FFlyingMovementAuxState& AuthorityState) const
{
	return false;
}

bool FFlyingMovementSyncState::ShouldReconcile(const FFlyingMovementSyncState& AuthorityState) const
{
	const float ErrorTolerance = FlyingPawnSimCVars::ErrorTolerance;
	UE_NP_TRACE_RECONCILE(!AuthorityState.Location.Equals(Location, ErrorTolerance), "Loc:");
	return false;
}

// -------------------------------------------------------------------------------------------------------

bool FFlyingMovementSimulation::ForceMispredict = false;
static FVector ForceMispredictVelocityMagnitude = FVector(2000.f, 0.f, 0.f);

bool IsExceedingMaxSpeed(const FVector& Velocity, float InMaxSpeed)
{
	InMaxSpeed = FMath::Max(0.f, InMaxSpeed);
	const float MaxSpeedSquared = FMath::Square(InMaxSpeed);
	
	// Allow 1% error tolerance, to account for numeric imprecision.
	const float OverVelocityPercent = 1.01f;
	return (Velocity.SizeSquared() > MaxSpeedSquared * OverVelocityPercent);
}

FVector ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit)
{
	return FVector::VectorPlaneProject(Delta, Normal) * Time;
}

void TwoWallAdjust(FVector& OutDelta, const FHitResult& Hit, const FVector& OldHitNormal)
{
	FVector Delta = OutDelta;
	const FVector HitNormal = Hit.Normal;

	if ((OldHitNormal | HitNormal) <= 0.f) //90 or less corner, so use cross product for direction
	{
		const FVector DesiredDir = Delta;
		FVector NewDir = (HitNormal ^ OldHitNormal);
		NewDir = NewDir.GetSafeNormal();
		Delta = (Delta | NewDir) * (1.f - Hit.Time) * NewDir;
		if ((DesiredDir | Delta) < 0.f)
		{
			Delta = -1.f * Delta;
		}
	}
	else //adjust to new wall
	{
		const FVector DesiredDir = Delta;
		Delta = ComputeSlideVector(Delta, 1.f - Hit.Time, HitNormal, Hit);
		if ((Delta | DesiredDir) <= 0.f)
		{
			Delta = FVector::ZeroVector;
		}
		else if ( FMath::Abs((HitNormal | OldHitNormal) - 1.f) < KINDA_SMALL_NUMBER )
		{
			// we hit the same wall again even after adjusting to move along it the first time
			// nudge away from it (this can happen due to precision issues)
			Delta += HitNormal * 0.01f;
		}
	}

	OutDelta = Delta;
}

float FFlyingMovementSimulation::SlideAlongSurface(const FVector& Delta, float Time, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	float PercentTimeApplied = 0.f;
	const FVector OldHitNormal = Normal;

	FVector SlideDelta = ComputeSlideVector(Delta, Time, Normal, Hit);

	if ((SlideDelta | Delta) > 0.f)
	{
		SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit, ETeleportType::None);

		const float FirstHitPercent = Hit.Time;
		PercentTimeApplied = FirstHitPercent;
		if (Hit.IsValidBlockingHit())
		{
			// Notify first impact
			if (bHandleImpact)
			{
				// !HandleImpact(Hit, FirstHitPercent * Time, SlideDelta);
			}

			// Compute new slide normal when hitting multiple surfaces.
			TwoWallAdjust(SlideDelta, Hit, OldHitNormal);

			// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
			if (!SlideDelta.IsNearlyZero(1e-3f) && (SlideDelta | Delta) > 0.f)
			{
				// Perform second move
				SafeMoveUpdatedComponent(SlideDelta, Rotation, true, Hit, ETeleportType::None);
				const float SecondHitPercent = Hit.Time * (1.f - FirstHitPercent);
				PercentTimeApplied += SecondHitPercent;

				// Notify second impact
				if (bHandleImpact && Hit.bBlockingHit)
				{
					// !HandleImpact(Hit, SecondHitPercent * Time, SlideDelta);
				}
			}
		}

		return FMath::Clamp(PercentTimeApplied, 0.f, 1.f);
	}

	return 0.f;
}

// bEnablePhysicsInteraction
// UCharacterMovementComponent::CapsuleTouched

void FFlyingMovementSimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<FlyingMovementStateTypes>& Input, const TNetSimOutput<FlyingMovementStateTypes>& Output)
{
	*Output.Sync = *Input.Sync;

	const float DeltaSeconds = (float)TimeStep.StepMS / 1000.f;

	// --------------------------------------------------------------
	//	Rotation Update
	//	We do the rotational update inside the movement sim so that things like server side teleport will work.
	//	(We want rotation to be treated the same as location, with respect to how its updated, corrected, etc).
	//	In this simulation, the rotation update isn't allowed to "fail". We don't expect the collision query to be able to fail the rotational update.
	// --------------------------------------------------------------

	Output.Sync->Rotation += (Input.Cmd->RotationInput * DeltaSeconds);
	Output.Sync->Rotation.Normalize();

	const FQuat OutputQuat = Output.Sync->Rotation.Quaternion();

	const FVector LocalSpaceMovementInput = Output.Sync->Rotation.RotateVector( Input.Cmd->MovementInput );
	   	
	// --------------------------------------------------------------
	// Calculate Output.Sync->RelativeVelocity based on Input
	// --------------------------------------------------------------
	{
		const FVector ControlAcceleration = LocalSpaceMovementInput.GetClampedToMaxSize(1.f);
		FVector Velocity = Input.Sync->Velocity;

		const float AnalogInputModifier = (ControlAcceleration.SizeSquared() > 0.f ? ControlAcceleration.Size() : 0.f);
		const float MaxPawnSpeed = Input.Aux->MaxSpeed * AnalogInputModifier;
		const bool bExceedingMaxSpeed = IsExceedingMaxSpeed(Velocity, MaxPawnSpeed);

		if (AnalogInputModifier > 0.f && !bExceedingMaxSpeed)
		{
			// Apply change in velocity direction
			if (Velocity.SizeSquared() > 0.f)
			{
				// Change direction faster than only using acceleration, but never increase velocity magnitude.
				const float TimeScale = FMath::Clamp(DeltaSeconds * Input.Aux->TurningBoost, 0.f, 1.f);
				Velocity = Velocity + (ControlAcceleration * Velocity.Size() - Velocity) * TimeScale;
			}
		}
		else
		{
			// Dampen velocity magnitude based on deceleration.
			if (Velocity.SizeSquared() > 0.f)
			{
				const FVector OldVelocity = Velocity;
				const float VelSize = FMath::Max(Velocity.Size() - FMath::Abs(Input.Aux->Deceleration) * DeltaSeconds, 0.f);
				Velocity = Velocity.GetSafeNormal() * VelSize;

				// Don't allow braking to lower us below max speed if we started above it.
				if (bExceedingMaxSpeed && Velocity.SizeSquared() < FMath::Square(MaxPawnSpeed))
				{
					Velocity = OldVelocity.GetSafeNormal() * MaxPawnSpeed;
				}
			}
		}

		// Apply acceleration and clamp velocity magnitude.
		const float NewMaxSpeed = (IsExceedingMaxSpeed(Velocity, MaxPawnSpeed)) ? Velocity.Size() : MaxPawnSpeed;
		Velocity += ControlAcceleration * FMath::Abs(Input.Aux->Acceleration) * DeltaSeconds;
		Velocity = Velocity.GetClampedToMaxSize(NewMaxSpeed);

		// Finally, output velocity that we calculated
		Output.Sync->Velocity = Velocity;
		
		if (FFlyingMovementSimulation::ForceMispredict)
		{
			Output.Sync->Velocity += ForceMispredictVelocityMagnitude;
			ForceMispredict = false;
		}
	}

	// ===================================================

	// --------------------------------------------------------------
	//	Calculate the final movement delta and move the update component
	// --------------------------------------------------------------
	FVector Delta = Output.Sync->Velocity * DeltaSeconds;

	if (!Delta.IsNearlyZero(1e-6f))
	{
		FHitResult Hit(1.f);
		SafeMoveUpdatedComponent(Delta, OutputQuat, true, Hit, ETeleportType::None);

		if (Hit.IsValidBlockingHit())
		{
			// Try to slide the remaining distance along the surface.
			SlideAlongSurface(Delta, 1.f-Hit.Time, OutputQuat, Hit.Normal, Hit, true);
		}

		// Update velocity
		// We don't want position changes to vastly reverse our direction (which can happen due to penetration fixups etc)
		/*
		if (!bPositionCorrected)
		{
			const FVector NewLocation = UpdatedComponent->GetComponentLocation();
			Velocity = ((NewLocation - OldLocation) / DeltaTime);
		}*/
	}

	// Finalize. This is unfortunate. The component mirrors our internal motion state and since we call into it to update, at this point, it has the real position.
	const FTransform UpdateComponentTransform = GetUpdateComponentTransform();
	Output.Sync->Location = UpdateComponentTransform.GetLocation();

	// Note that we don't pull the rotation out of the final update transform. Converting back from a quat will lead to a different FRotator than what we are storing
	// here in the simulation layer. This may not be the best choice for all movement simulations, but is ok for this one.
}

void FFlyingMovementSimulation::OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult)
{	

}
