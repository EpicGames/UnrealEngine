// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockAbilitySimulation.h"
#include "Kismet/KismetSystemLibrary.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "DrawDebugHelpers.h"
#include "NetworkPredictionProxyInit.h"
#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

namespace MockAbilityCVars
{
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DefaultMaxSpeed, 1200.f, "mockability.DefaultMaxSpeed", "Default Speed");
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DefaultAcceleration, 4000.f, "mockability.DefaultAcceleration", "Default Acceleration");

	NETSIM_DEVCVAR_SHIPCONST_FLOAT(SprintMaxSpeed, 5000.f, "mockability.SprintMaxSpeed", "Max Speed when sprint is applied.");

	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DashMaxSpeed, 7500.f, "mockability.DashMaxSpeed", "Max Speed when dashing.");
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DashAcceleration, 100000.f, "mockability.DashAcceleration", "Acceleration when dashing.");


	NETSIM_DEVCVAR_SHIPCONST_INT(BlinkCueType, 4, "mockability.BlinkCueType", "0=Skip. 1=weak. 2=ReplicatedNonPredicted, 3=ReplicatedXOrPredicted, 4=Strong");
	NETSIM_DEVCVAR_SHIPCONST_INT(BlinkWarmupMS, 750, "mockability.BlinkWarmupMS", "Duration in MS of blink warmup period");

	NETSIM_DEVCVAR_SHIPCONST_INT(DisablePhysicsGunServer, 0, "mockability.DisablePhysicsGunServer", "Disables gravity gun on server, causing mispredictions");
	NETSIM_DEVCVAR_SHIPCONST_INT(DisablePhysicsGunClient, 0, "mockability.DisablePhysicsGubClient", "Disables gravity gun on client, causing mispredictions");

	NETSIM_DEVCVAR_SHIPCONST_INT(DisablePhysicsGunCues, 0, "mockability.DisablePhysicsGunCues", "Disables gravity gun cues. Less explosions :(");
	NETSIM_DEVCVAR_SHIPCONST_INT(PhysicsGunCooldown, 500, "mockability.PhysicsGunCooldown", "Physics gun cooldown in MS");
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(PhysicsGunImpulse, 300000, "mockability.PhysicsGunImpulse", "Physics gun impulse");
}

// -------------------------------------------------------------------------------------------------------------
//
// -------------------------------------------------------------------------------------------------------------

class FMockAbilityModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = FMockAbilitySimulation;
	using StateTypes = TMockAbilityBufferTypes;
	using Driver = UMockFlyingAbilityComponent;

	/*
	static void Interpolate(const TInterpolatorParameters<FMockAbilitySyncState, FMockAbilityAuxState>& Params)
	{
		Params.Out.Sync = Params.To.Sync;
		Params.Out.Aux = Params.To.Aux;

		FFlyingMovementNetSimModelDef::Interpolate(Params.Cast<FFlyingMovementSyncState, FFlyingMovementAuxState>());
	}
	*/

	static const TCHAR* GetName() { return TEXT("MockAbilityModelDef"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers + 2; }
};

bool FMockAbilitySyncState::ShouldReconcile(const FMockAbilitySyncState& AuthorityState) const
{
	UE_NP_TRACE_RECONCILE(Stamina != AuthorityState.Stamina, "Stamina:");

	return FFlyingMovementSyncState::ShouldReconcile(AuthorityState);
}

bool FMockAbilityAuxState::ShouldReconcile(const FMockAbilityAuxState& AuthorityState) const
{
	UE_NP_TRACE_RECONCILE(MaxStamina != AuthorityState.MaxStamina, "MaxStamina:");
	UE_NP_TRACE_RECONCILE(StaminaRegenRate != AuthorityState.StaminaRegenRate, "StaminaRegenRate:");
	UE_NP_TRACE_RECONCILE(DashTimeLeft != AuthorityState.DashTimeLeft, "DashTimeLeft:");
	UE_NP_TRACE_RECONCILE(BlinkWarmupLeft != AuthorityState.BlinkWarmupLeft, "BlinkWarmupLeft:");
	UE_NP_TRACE_RECONCILE(PrimaryCooldown != AuthorityState.PrimaryCooldown, "PrimaryCooldown:");
	UE_NP_TRACE_RECONCILE(bIsSprinting != AuthorityState.bIsSprinting, "bIsSprinting:");
	
	return FFlyingMovementAuxState::ShouldReconcile(AuthorityState);
}

NP_MODEL_REGISTER(FMockAbilityModelDef);

NETSIMCUE_REGISTER(FMockAbilityBlinkActivateCue, TEXT("MockAbilityBlinkActivate"));
NETSIMCUE_REGISTER(FMockAbilityBlinkCue, TEXT("MockAbilityBlink"));
NETSIMCUE_REGISTER(FMockAbilityPhysicsGunFireCue, TEXT("MockAbilityPhysicsGunFireCue"));

NETSIMCUE_REGISTER(FMockAbilityBlinkCue_Weak, TEXT("MockAbilityBlink_Weak"));
NETSIMCUE_REGISTER(FMockAbilityBlinkCue_ReplicatedNonPredicted, TEXT("MockAbilityBlink_RepNonPredicted"));
NETSIMCUE_REGISTER(FMockAbilityBlinkCue_ReplicatedXOrPredicted, TEXT("MockAbilityBlink_RepXOrPredicted"));
NETSIMCUE_REGISTER(FMockAbilityBlinkCue_Strong, TEXT("MockAbilityBlink_Strong"));

NETSIMCUESET_REGISTER(UMockFlyingAbilityComponent, FMockAbilityCueSet);

// -------------------------------------------------------------------------------------------------------------
//
// -------------------------------------------------------------------------------------------------------------

void FMockAbilitySimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockAbilityBufferTypes>& Input, const TNetSimOutput<TMockAbilityBufferTypes>& Output)
{
	// WIP NOTES:
	//	-We are creating local copies of the input state that we can mutate and pass on to the parent simulation's tick (LocalCmd, LocalSync, LocalAux). This may seem weird but consider:
	//		-Modifying the actual inputs is a bad idea: inputs are "final" once TNetworkedSimulationModel calls SimulationTick. The sim itself should not modify the inputs after this.
	//		-Tempting to just use the already-allocated Output states and pass them to the parent sim as both Input/Output. But now &Input.Sync == &Output.Sync for example! The parent sim
	//			may write to Output and then later check it against the passed in input (think "did this change"), having no idea that by writing to output it was also writing to input! This seems like it should be avoided.
	//		-It may be worth considering "lazy copying" or allocating of these temp states... but for now that adds more complexity and isn't necessary since the local copies are stack allocated and
	//			these state structures are small and contiguous.
	//
	//	-Still, there are some awkwardness when we need to write to both the local temp state *and* the output state explicitly.
	//	-Stamina changes for example: we want to both change the stamina both for this frame (used by rest of code in this function) AND write it to the output state (so it comes in as input next frame)
	//		-Since this sim "owns" stamina, we could copy it immediately to the output state and use the output state exclusively in this code...
	//		-... but that is inconsistent with state that we don't "own" like Location. Since the parent sim will generate a new Location and we can't pass output to the parent as input
	//			-You would be in a situation where "Teleport modified LocalSync.Location" and "another place modified Output.Sync->Stamina".
	//	
	//
	//	Consider: not using inheritance for simulations. Would that make this less akward? 
	//		-May make Sync state case above less weird since it would breakup stamina/location ("owned" vs "not owned"). Stamina could always use output state and we'd have a local copy of the movement input sync state.
	//		-Aux state is still bad even without inheritance: if you want to read and write to an aux variable at different points in the frames, hard to know where "real" value is at any given time without
	//			writing fragile code. Making code be explicit: "I am writing to the current value this frame AND next frames value" seems ok. Could possibly do everything in a local copy and then see if its changed
	//			at the end up ::SimulationTick and then copy it into output aux. Probably should avoid comparison operator each frame so would need to wrap in some struct that could track a dirty flag... seems complicating.
	//

	const float DeltaSeconds = (float)TimeStep.StepMS / 1000.f;

	// Local copies of the const input that we will pass into the parent sim as input.
	FMockAbilityInputCmd LocalCmd = *Input.Cmd;
	FMockAbilitySyncState LocalSync = *Input.Sync;
	FMockAbilityAuxState LocalAux = *Input.Aux;

	const bool bAlreadyBlinking = Input.Aux->BlinkWarmupLeft > 0;
	const bool bAlreadyDashing = Input.Aux->DashTimeLeft > 0;
	const bool bAllowNewActivations = !bAlreadyBlinking && !bAlreadyDashing;
	
	// -------------------------------------------------------------------------
	//	Regen
	//	-Applies at the start of frame: you can spend it immediately
	//	-hence we have to write to both the LocalSync's stamina (what the rest of the abilities look at for input) and output (what value will be used as input for next frame)
	// -------------------------------------------------------------------------

	{
		const float NewStamina = FMath::Min<float>( LocalSync.Stamina + (DeltaSeconds * LocalAux.StaminaRegenRate), LocalAux.MaxStamina );
		LocalSync.Stamina = NewStamina;
		Output.Sync->Stamina = NewStamina;
	}

	// -------------------------------------------------------------------------
	//	Blink
	// -------------------------------------------------------------------------

	static float BlinkCost = 25.f;
	const int16 BlinkWarmupMS = MockAbilityCVars::BlinkWarmupMS();

	auto GetBlinkDestination = [&LocalSync]()
	{
		static float BlinkDist = 1000.f;
		const FVector DestLocation = LocalSync.Location + LocalSync.Rotation.RotateVector( FVector(BlinkDist, 0.f, 0.f) );
		return DestLocation;
	};

	const bool bBlinkActivate = (Input.Cmd->bBlinkPressed && Input.Sync->Stamina > BlinkCost && bAllowNewActivations);
	if(bBlinkActivate)
	{
		LocalAux.BlinkWarmupLeft = BlinkWarmupMS;
		
		// Invoke a cue to telegraph where the blink will land. This is making the assumption the handler wouldn't either want to or be able to derive the blink destination from the current state alone
		// The randomValue being calculated here is purposefully to cause mis prediction of the cue, so that we can demonstrate rollback -> resimulate can do the correction seemlessly
		
		UE_LOGF(LogNetworkPrediction, Warning, "Invoking FMockAbilityBlinkActivateCue from sim");
		Output.CueDispatch.Invoke<FMockAbilityBlinkActivateCue>( GetBlinkDestination(), FMath::Rand() % 255 );
	}
	
	if (LocalAux.BlinkWarmupLeft > 0)
	{
		const int16 NewBlinkWarmupLeft = FMath::Max<int16>(0, LocalAux.BlinkWarmupLeft - TimeStep.StepMS);
		Output.Aux.Get()->BlinkWarmupLeft = NewBlinkWarmupLeft;

		// While blinking is warming up, we disable movement input and the other actions

		LocalCmd.MovementInput.Set(0.f, 0.f, 0.f);
		LocalCmd.bDashPressed = false;
		LocalCmd.bSprintPressed = false;
		LocalCmd.RotationInput = FRotator::ZeroRotator;
		//LocalSync.Rotation = Input.Sync->Rotation;
		LocalSync.Velocity.Set(0.f, 0.f, 0.f);

		if (NewBlinkWarmupLeft <= 0)
		{
			const FVector DestLocation = GetBlinkDestination();
			AActor* OwningActor = UpdatedComponent->GetOwner();
			check(OwningActor);

			// DrawDebugLine(OwningActor->GetWorld(), Input.Sync->Location, DestLocation, FColor::Red, false);	

			// Its unfortunate teleporting is so complicated. It may make sense for a new movement simulation to define this themselves, 
			// but for this mock one, we will just use the engine's AActor teleport.
			
			if (OwningActor->TeleportTo(DestLocation, LocalSync.Rotation))
			{
				// Component now has the final location
				const FTransform UpdateComponentTransform = GetUpdateComponentTransform();
				LocalSync.Location = UpdateComponentTransform.GetLocation();
				
				const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - BlinkCost);
				LocalSync.Stamina = NewStamina;
				Output.Sync->Stamina = NewStamina;

				// Invoke a NetCue for the blink. This is essentially capturing the start/end location so that all receivers of the event
				// get the exact coordinates (maybe overkill in practice but key is that we have data that we want to pass out via an event)
				
				//UE_LOGF(LogNetworkPrediction, Warning, "Invoking FMockAbilityBlinkCue from sim. %ls - %ls (LocalSync.Location: %ls) ", *Input.Sync->Location.ToString(), *DestLocation.ToString(), *LocalSync.Location.ToString());
				//UE_VLOG(OwningActor, LogNetworkPrediction, Log, TEXT("Invoking FMockAbilityBlinkCue from sim. %s - %s (LocalSync.Location: %s) "), *Input.Sync->Location.ToString(), *DestLocation.ToString(), *LocalSync.Location.ToString());

				switch (MockAbilityCVars::BlinkCueType()) // Only for dev/testing. Not a normal setup.
				{
				case 0:
					// Skip on purpose
					break;
				case 1:
					Output.CueDispatch.Invoke<FMockAbilityBlinkCue_Weak>( Input.Sync->Location, DestLocation );
					break;
				case 2:
					Output.CueDispatch.Invoke<FMockAbilityBlinkCue_ReplicatedNonPredicted>( Input.Sync->Location, DestLocation );
					break;

				case 3:
					Output.CueDispatch.Invoke<FMockAbilityBlinkCue_ReplicatedXOrPredicted>( Input.Sync->Location, DestLocation );
					break;

				case 4:
					Output.CueDispatch.Invoke<FMockAbilityBlinkCue_Strong>( Input.Sync->Location, DestLocation );
					break;

				};
			}
		}
	}

	// -------------------------------------------------------------------------
	//	Dash
	//	-Stamina consumed on initial press
	//	-MaxSpeed/Acceleration are jacked up
	//	-Dash lasts for 400ms (DashDurationMS)
	//		-Division of frame times can cause you to dash for longer. We would have to break up simulation steps to support this 100% accurately.
	//	-Movement input is synthesized while in dash state. That is, we force forward movement and ignore what was actually fed into the simulation (move input only)
	// -------------------------------------------------------------------------

	static float DashCost = 25.f;
	static int16 DashDurationMS = 500;
	
	const bool bDashActivate = (Input.Cmd->bDashPressed && Input.Sync->Stamina > DashCost && bAllowNewActivations);
	if (bDashActivate)
	{
		// Start dashing
		LocalAux.DashTimeLeft = DashDurationMS;
		
		const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - DashCost);
		LocalSync.Stamina = NewStamina;
		Output.Sync->Stamina = NewStamina;
	}

	if (LocalAux.DashTimeLeft > 0)
	{
		const int16 NewDashTimeLeft = FMath::Max<int16>(LocalAux.DashTimeLeft - (int16)(DeltaSeconds * 1000.f), 0);
		Output.Aux.Get()->DashTimeLeft = NewDashTimeLeft;

		LocalAux.MaxSpeed = MockAbilityCVars::DashMaxSpeed();
		LocalAux.Acceleration = MockAbilityCVars::DashAcceleration();

		LocalCmd.MovementInput.Set(1.f, 0.f, 0.f);
		LocalCmd.bBlinkPressed = false;
		LocalCmd.bSprintPressed = false;
		
		FFlyingMovementSimulation::SimulationTick(TimeStep, { &LocalCmd, &LocalSync, &LocalAux }, { Output.Sync, Output.Aux, Output.CueDispatch });

		if (NewDashTimeLeft == 0)
		{
			// Stop when dash is over
			Output.Sync->Velocity.Set(0.f, 0.f, 0.f);
		}

		return;
	}


	// -------------------------------------------------------------------------
	//	Sprint
	//	-Requires 15% stamina to start activation
	//	-Drains at 100 stamina/second after that
	//	-Just bumps up max move speed
	//	-Note how this is a transient value: it is fed into the parent sim's input but never makes it to an output state
	// -------------------------------------------------------------------------
	
	static float SprintBaseCost = 100.f; // 100 sprint/second cost
	static float SprintStartMinPCT = 0.15f; // 15% stamina required to begin sprinting
	
	bool bIsSprinting = false;
	if (Input.Cmd->bSprintPressed && bAllowNewActivations)
	{
		const float SprintCostThisFrame = SprintBaseCost * DeltaSeconds;

		if (Input.Aux->bIsSprinting)
		{
			// If we are already sprinting, then we keep sprinting as long as we can afford the absolute cost
			bIsSprinting = LocalSync.Stamina > SprintCostThisFrame;
		}
		else
		{
			// To start sprinting, we have to have SprintStartMinPCT stamina
			bIsSprinting = LocalSync.Stamina > (SprintStartMinPCT * LocalAux.MaxStamina);
		}

		if (bIsSprinting)
		{
			LocalAux.MaxSpeed = MockAbilityCVars::SprintMaxSpeed();

			const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - SprintCostThisFrame);
			LocalSync.Stamina = NewStamina;
			Output.Sync->Stamina = NewStamina;

		}
	}

	// Update the out aux state and call notifies only if sprinting state as actually changed
	if (bIsSprinting != Input.Aux->bIsSprinting)
	{
		Output.Aux.Get()->bIsSprinting = bIsSprinting;
	}

	FFlyingMovementSimulation::SimulationTick(TimeStep, { &LocalCmd, &LocalSync, &LocalAux }, { Output.Sync, Output.Aux, Output.CueDispatch });

	// Physics gun (run this last so we have the latest Location)
	if (LocalAux.PrimaryCooldown > 0)
	{
		const int16 NewPrimaryCooldown = FMath::Max<int16>(0, LocalAux.PrimaryCooldown - TimeStep.StepMS);
		Output.Aux.Get()->PrimaryCooldown = NewPrimaryCooldown;
	}

	const bool bDisablePhysicsGun = UpdatedComponent->GetOwnerRole() == ROLE_Authority ? MockAbilityCVars::DisablePhysicsGunServer() > 0: MockAbilityCVars::DisablePhysicsGunClient() > 0;

	if (LocalCmd.bPrimaryPressed && LocalAux.PrimaryCooldown <= 0 && !bDisablePhysicsGun)
	{
		const FVector Start = Output.Sync->Location;
		const FVector End = Start + Output.Sync->Rotation.RotateVector( FVector(5000, 0.f, 0.f) );
		const float Radius = 100.f;
		const float Strength = -1.f * MockAbilityCVars::PhysicsGunImpulse();

		TArray<TEnumAsByte<EObjectTypeQuery> > ObjectTypes { UEngineTypes::ConvertToObjectType(ECollisionChannel::ECC_PhysicsBody) };
		TArray<AActor*> ActorsToIgnore { UpdatedComponent->GetOwner() };
		TArray<UPrimitiveComponent*> OutComponents;
		TArray<FHitResult> OutHits;

		TArray<FVector_NetQuantize100> HitLocations;
		
		if (UKismetSystemLibrary::SphereTraceMultiForObjects(UpdatedComponent, Start, End, Radius, ObjectTypes, false, ActorsToIgnore, EDrawDebugTrace::None, OutHits, true))
		{
			for (FHitResult& Hit : OutHits)
			{
				if (UPrimitiveComponent* HitComponent = Hit.Component.Get())
				{
					// Probably need to expand this: "am I allowed to add impulses to this component in this network context"
					if (HitComponent->IsSimulatingPhysics())
					{
						UE_LOGF(LogNetworkPrediction, Warning, "[%d] %ls. PhysicsGun Adding Impulses! ImpactPoint: %ls", TimeStep.Frame, UpdatedComponent->GetOwnerRole() == ROLE_Authority ? TEXT("Server") : TEXT("Client"), *Hit.ImpactPoint.ToString());
						HitComponent->AddImpulseAtLocation(Hit.ImpactNormal * Strength, Hit.ImpactPoint);
					}

					HitLocations.Add(Hit.ImpactPoint);
				}
			}
		}

		// Every frame Cue is probably not a good idea but with CD its ok
		if (!MockAbilityCVars::DisablePhysicsGunCues())
		{
			Output.CueDispatch.Invoke<FMockAbilityPhysicsGunFireCue>( Start, End, true, MoveTemp(HitLocations) );
		}

		Output.Aux.Get()->PrimaryCooldown = MockAbilityCVars::PhysicsGunCooldown();
	}

	if (LocalCmd.bSecondaryPressed && !(MockAbilityCVars::DisablePhysicsGunServer() && UpdatedComponent->GetOwnerRole() == ROLE_Authority))
	{
		const FVector Start = Output.Sync->Location;
		const FVector End = Start + Output.Sync->Rotation.RotateVector( FVector(5000, 0.f, 0.f) );
		const float Radius = 100.f;
		const float Strength = -0.2f * MockAbilityCVars::PhysicsGunImpulse();

		TArray<TEnumAsByte<EObjectTypeQuery> > ObjectTypes { UEngineTypes::ConvertToObjectType(ECollisionChannel::ECC_PhysicsBody) };
		TArray<AActor*> ActorsToIgnore { UpdatedComponent->GetOwner() };
		TArray<UPrimitiveComponent*> OutComponents;
		TArray<FHitResult> OutHits;

		TArray<FVector_NetQuantize100> HitLocations;

		if (UKismetSystemLibrary::SphereTraceMultiForObjects(UpdatedComponent, Start, End, Radius, ObjectTypes, false, ActorsToIgnore, EDrawDebugTrace::None, OutHits, true))
		{
			for (FHitResult& Hit : OutHits)
			{
				if (UPrimitiveComponent* HitComponent = Hit.Component.Get())
				{
					// Probably need to expand this: "am I allowed to add impulses to this component in this network context"
					if (HitComponent->IsSimulatingPhysics())
					{
						UE_LOGF(LogNetworkPrediction, Warning, "[%d] %ls. PhysicsGun Adding Impulses! ImpactPoint: %ls", TimeStep.Frame, UpdatedComponent->GetOwnerRole() == ROLE_Authority ? TEXT("Server") : TEXT("Client"), *Hit.ImpactPoint.ToString());
						HitComponent->AddImpulseAtLocation(Hit.ImpactNormal * Strength, Hit.ImpactPoint);
					}

					HitLocations.Add(Hit.ImpactPoint);
				}
			}
		}

		// Every frame Cue is probably not a good idea but with CD its ok
		if (!MockAbilityCVars::DisablePhysicsGunCues())
		{
			Output.CueDispatch.Invoke<FMockAbilityPhysicsGunFireCue>( Start, End, false, MoveTemp(HitLocations) );
		}

		/*
		const FVector SpherePosition = LocalSync.Location;
		const float SphereRadius = 1000.f;
		TArray<TEnumAsByte<EObjectTypeQuery> > ObjectTypes { UEngineTypes::ConvertToObjectType(ECollisionChannel::ECC_PhysicsBody) };
		TArray<UPrimitiveComponent*> OverlapComponents;
		TArray<AActor*> ActorsToIgnore { UpdatedComponent->GetOwner() };
		TArray<UPrimitiveComponent*> OutComponents;

		const float Strength = -1000000;

		if (UKismetSystemLibrary::SphereOverlapComponents(UpdatedComponent, SpherePosition, SphereRadius, ObjectTypes, nullptr, ActorsToIgnore, OutComponents))
		{
			for (UPrimitiveComponent* HitComponent : OutComponents)
			{
				UE_LOGF(LogNetworkPrediction, Warning, "Hit: %ls", *GetPathNameSafe(HitComponent));
				HitComponent->AddRadialForce(SpherePosition, SphereRadius, Strength, ERadialImpulseFalloff::RIF_Linear, false);
			}
		}
		*/
	}

	/*
	AActor* OwningActor = UpdatedComponent->GetOwner();
	UWorld* World = OwningActor->GetWorld();

	static FVector Force(0.f, 0.f, 1000.f);

	UE_LOGF(LogNetworkPrediction, Warning, "[%d] Adding Impulses! %ls", TimeStep.Frame, OwningActor->GetLocalRole() == ROLE_Authority ? TEXT("Server") : TEXT("Client"));
	for (TActorIterator<AActor> It(World); It; ++It)
	{
		AActor* TestActor = *It;
		if (UPrimitiveComponent* Prim = TestActor->FindComponentByClass<UPrimitiveComponent>())
		{
			if (Prim->IsAnySimulatingPhysics())
			{
				//UE_LOGF(LogMockNetworkSim, Warning, "Adding Impulses! %ls  %f / %f", OwningActor->GetLocalRole() == ROLE_Authority ? TEXT("Server") : TEXT("Client"), Prim->GetOwner()->GetActorLocation().Z, PhysVec.Z);
				Prim->AddImpulse(Force, NAME_None, true);
			}
		}
	}
	*/
}

// -------------------------------------------------------------------------------------------------------------
//
// -------------------------------------------------------------------------------------------------------------

UMockFlyingAbilityComponent::UMockFlyingAbilityComponent()
{

}

void UMockFlyingAbilityComponent::InitializeNetworkPredictionProxy()
{
	check(UpdatedComponent);

	OwnedAbilitySimulation = MakePimpl<FMockAbilitySimulation>();
	InitMockAbilitySimulation(OwnedAbilitySimulation.Get());

	NetworkPredictionProxy.Init<FMockAbilityModelDef>(GetWorld(), GetReplicationProxies(), OwnedAbilitySimulation.Get(), this);
}

void UMockFlyingAbilityComponent::InitMockAbilitySimulation(FMockAbilitySimulation* Simulation)
{
	ActiveAbilitySimulation = Simulation;
	InitFlyingMovementSimulation(Simulation);
}

void UMockFlyingAbilityComponent::InitializeSimulationState(FMockAbilitySyncState* SyncState, FMockAbilityAuxState* AuxState)
{
	UFlyingMovementComponent::InitializeSimulationState(SyncState, AuxState);
}

void UMockFlyingAbilityComponent::ProduceInput(const int32 SimTimeMS, FMockAbilityInputCmd* Cmd)
{
	// This isn't ideal. It probably makes sense for the component to do all the input binding rather.
	ProduceInputDelegate.ExecuteIfBound(SimTimeMS, *Cmd);
}

void UMockFlyingAbilityComponent::FinalizeFrame(const FMockAbilitySyncState* SyncState, const FMockAbilityAuxState* AuxState)
{
	npCheckSlow(SyncState);
	npCheckSlow(AuxState);

	UFlyingMovementComponent::FinalizeFrame(SyncState, AuxState);

	if (AuxState->bIsSprinting != bIsSprinting)
	{
		bIsSprinting = AuxState->bIsSprinting;
		OnSprintStateChange.Broadcast(AuxState->bIsSprinting);
	}

	const bool bLocalIsDashing = (AuxState->DashTimeLeft > 0);
	if (bLocalIsDashing != bIsDashing)
	{
		bIsDashing = bLocalIsDashing;
		OnDashStateChange.Broadcast(bIsDashing);
	}

	const bool bLocalIsBlinking = (AuxState->BlinkWarmupLeft > 0);
	if (bLocalIsBlinking != bIsBlinking)
	{
		bIsBlinking = bLocalIsBlinking;
		OnBlinkStateChange.Broadcast(bLocalIsBlinking);
	}
}

float UMockFlyingAbilityComponent::GetBlinkWarmupTimeSeconds() const
{
	return MockAbilityCVars::BlinkWarmupMS() / 1000.f;
}

float UMockFlyingAbilityComponent::GetStamina() const
{
	if (const FMockAbilitySyncState* Sync = NetworkPredictionProxy.ReadSyncState<FMockAbilitySyncState>())
	{
		return Sync->Stamina;
	}
	return 0.f;
}

float UMockFlyingAbilityComponent::GetMaxStamina() const
{
	if (const FMockAbilityAuxState* Aux = NetworkPredictionProxy.ReadAuxState<FMockAbilityAuxState>())
	{
		return Aux->MaxStamina;
	}
	return 0.f;
}

// ---------------------------------------------------------------------------------

void UMockFlyingAbilityComponent::HandleCue(const FMockAbilityBlinkActivateCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters)
{
	FString RoleStr = GetOwnerRole() == ROLE_Authority ? TEXT("Server") : GetOwnerRole() == ROLE_SimulatedProxy ? TEXT("SP Client") : TEXT("AP Client");
	UE_LOGF(LogNetworkPrediction, Display, "[%ls] BlinkActivatedCue! TimeMS: %d", *RoleStr, SystemParameters.TimeSinceInvocation);
	
	this->OnBlinkActivateEvent.Broadcast(BlinkCue.Destination, BlinkCue.RandomType, (float)SystemParameters.TimeSinceInvocation / 1000.f);
	
	if (SystemParameters.Callbacks)
	{
		UE_LOGF(LogNetworkPrediction, Display, "  System Callbacks available!");

		SystemParameters.Callbacks->OnRollback.AddLambda([RoleStr, this]()
		{
			UE_LOGF(LogNetworkPrediction, Display, "  %ls BlinkActivatedCue Rollback!", *RoleStr);
			this->OnBlinkActivateEventRollback.Broadcast();
		});
	}
}

static float BlinkCueDuration = 1.f;
static FAutoConsoleVariableRef CVarBindAutomatically(TEXT("NetworkPredictionExtras.FlyingPawn.BlinkCueDuration"),
	BlinkCueDuration, TEXT("Duration of BlinkCue"), ECVF_Default);

void UMockFlyingAbilityComponent::HandleCue(const FMockAbilityBlinkCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters)
{
	FString RoleStr = *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwnerRole());
	FVector Delta = GetOwner()->GetActorLocation() - BlinkCue.StopLocation;

	UE_LOGF(LogNetworkPrediction, Display, "[%ls] BlinkCue! : <%f, %f, %f> - <%f, %f, %f>. ElapsedTimeMS: %d. Delta: %.3f", *RoleStr, BlinkCue.StartLocation.X, BlinkCue.StartLocation.Y, BlinkCue.StartLocation.Z,
		BlinkCue.StopLocation.X, BlinkCue.StopLocation.Y, BlinkCue.StopLocation.Z, SystemParameters.TimeSinceInvocation, Delta.Size()); //*BlinkCue.StartLocation.ToString(), *BlinkCue.StopLocation.ToString());

	// Crude compensation for cue firing in the past (note this is not necessary! Some cues not care and need to see the "full" effect regardless of when it happened)
	float Duration = FMath::Max<float>(0.1f, BlinkCueDuration - ((float)SystemParameters.TimeSinceInvocation/1000.f));
	DrawDebugLine(GetWorld(), BlinkCue.StartLocation, BlinkCue.StopLocation, (FMath::Rand() % 2) == 0 ? FColor::Red : FColor::Blue, false, Duration);

	if (SystemParameters.Callbacks)
	{
		UE_LOGF(LogNetworkPrediction, Display, "  System Callbacks available!");

		SystemParameters.Callbacks->OnRollback.AddLambda([RoleStr, this]()
		{
			UE_LOGF(LogNetworkPrediction, Display, "  %ls BlinkCue Rollback!", *RoleStr);
		});
	}
}

void UMockFlyingAbilityComponent::HandleCue(const FMockAbilityPhysicsGunFireCue& FireCue, const FNetSimCueSystemParamemters& SystemParameters)
{
	OnPhysicsGunFirEvent.Broadcast(FireCue.Start, FireCue.End, FireCue.bHasCooldown, FireCue.HitLocations, (float)SystemParameters.TimeSinceInvocation / 1000.f);
}
