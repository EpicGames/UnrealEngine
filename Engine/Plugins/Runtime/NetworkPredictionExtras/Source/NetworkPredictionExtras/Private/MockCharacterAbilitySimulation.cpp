// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockCharacterAbilitySimulation.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionModelDefRegistry.h"

#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

DEFINE_LOG_CATEGORY_STATIC(LogCharacterAbilitySimulation, Log, All);


namespace MockCharacterAbilityCVars
{
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DefaultMaxSpeed, 1200.f, "mockcharacterability.DefaultMaxSpeed", "Default Speed");
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DefaultAcceleration, 4000.f, "mockcharacterability.DefaultAcceleration", "Default Acceleration");

	NETSIM_DEVCVAR_SHIPCONST_FLOAT(SprintMaxSpeed, 5000.f, "mockcharacterability.SprintMaxSpeed", "Max Speed when sprint is applied.");

	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DashMaxSpeed, 7500.f, "mockcharacterability.DashMaxSpeed", "Max Speed when dashing.");
	NETSIM_DEVCVAR_SHIPCONST_FLOAT(DashAcceleration, 100000.f, "mockcharacterability.DashAcceleration", "Acceleration when dashing.");

	//NETSIM_DEVCVAR_SHIPCONST_INT(BlinkCueType, 4, "mockcharacterability.BlinkCueType", "0=Skip. 1=weak. 2=ReplicatedNonPredicted, 3=ReplicatedXOrPredicted, 4=Strong");
	NETSIM_DEVCVAR_SHIPCONST_INT(BlinkWarmupMS, 750, "mockcharacterability.BlinkWarmupMS", "Duration in MS of blink warmup period");
}


// -------------------------------------------------------------------------------------------------------------
//
// -------------------------------------------------------------------------------------------------------------

class FMockCharacterAbilityModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = FMockCharacterAbilitySimulation;
	using StateTypes = TMockCharacterAbilityBufferTypes;
	using Driver = UMockCharacterAbilityComponent;

	/*
	static void Interpolate(const TInterpolatorParameters<FMockCharacterAbilitySyncState, FMockCharacterAbilityAuxState>& Params)
	{
		Params.Out.Sync = Params.To.Sync;
		Params.Out.Aux = Params.To.Aux;

		FCharacterMotionNetSimModelDef::Interpolate(Params.Cast<FCharacterMotionSyncState, FCharacterMotionAuxState>());
	}
	*/

	static const TCHAR* GetName() { return TEXT("MockCharacterAbilityModelDef"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers + 2; }
};

bool FMockCharacterAbilitySyncState::ShouldReconcile(const FMockCharacterAbilitySyncState& AuthorityState) const
{
	if (Stamina != AuthorityState.Stamina)
	{
		return true;
	}

	return FCharacterMotionSyncState::ShouldReconcile(AuthorityState);
}

bool FMockCharacterAbilityAuxState::ShouldReconcile(const FMockCharacterAbilityAuxState& AuthorityState) const
{
	if (MaxStamina != AuthorityState.MaxStamina
		|| StaminaRegenRate != AuthorityState.StaminaRegenRate
		|| DashTimeLeft != AuthorityState.DashTimeLeft
		|| BlinkWarmupLeft != AuthorityState.BlinkWarmupLeft
		|| PrimaryCooldown != AuthorityState.PrimaryCooldown
		|| bIsSprinting != AuthorityState.bIsSprinting
		|| bIsJumping != AuthorityState.bIsJumping)
	{
		return true;
	}

	return FCharacterMotionAuxState::ShouldReconcile(AuthorityState);
}

NP_MODEL_REGISTER(FMockCharacterAbilityModelDef);


UMockCharacterAbilityComponent::UMockCharacterAbilityComponent()
{

}

void UMockCharacterAbilityComponent::InitializeNetworkPredictionProxy()
{
	check(UpdatedComponent);

	OwnedAbilitySimulation = MakePimpl<FMockCharacterAbilitySimulation>();
	InitMockCharacterAbilitySimulation(OwnedAbilitySimulation.Get());

	NetworkPredictionProxy.Init<FMockCharacterAbilityModelDef>(GetWorld(), GetReplicationProxies(), OwnedAbilitySimulation.Get(), this);
}

void UMockCharacterAbilityComponent::InitMockCharacterAbilitySimulation(FMockCharacterAbilitySimulation* Simulation)
{
	ActiveAbilitySimulation = Simulation;
	InitCharacterMotionSimulation(Simulation);
}

void UMockCharacterAbilityComponent::InitializeSimulationState(FMockCharacterAbilitySyncState* SyncState, FMockCharacterAbilityAuxState* AuxState)
{
	Super::InitializeSimulationState(SyncState, AuxState);
}

void UMockCharacterAbilityComponent::ProduceInput(const int32 SimTimeMS, FMockCharacterAbilityInputCmd* Cmd)
{
	// This isn't ideal. It probably makes sense for the component to do all the input binding rather.
	ProduceInputDelegate.ExecuteIfBound(SimTimeMS, *Cmd);
}

void UMockCharacterAbilityComponent::FinalizeFrame(const FMockCharacterAbilitySyncState* SyncState, const FMockCharacterAbilityAuxState* AuxState)
{
	npCheckSlow(SyncState);
	npCheckSlow(AuxState);

	UCharacterMotionComponent::FinalizeFrame(SyncState, AuxState);

	if (AuxState->bIsSprinting != bIsSprinting)
	{
		bIsSprinting = AuxState->bIsSprinting;
		OnSprintStateChange.Broadcast(AuxState->bIsSprinting);
	}

	const bool bLocalIsDashing = (AuxState->DashTimeLeft > 0);
	if (bLocalIsDashing != bIsDashing)
	{
		bIsDashing = bLocalIsDashing;
		OnDashStateChange.Broadcast(bIsDashing);
	}

	const bool bLocalIsBlinking = (AuxState->BlinkWarmupLeft > 0);
	if (bLocalIsBlinking != bIsBlinking)
	{
		bIsBlinking = bLocalIsBlinking;
		OnBlinkStateChange.Broadcast(bLocalIsBlinking);
	}

	const bool bLocalIsJumping = (AuxState->bIsJumping);
	if (bLocalIsJumping != bIsJumping)
	{
		bIsJumping = bLocalIsJumping;
		OnJumpStateChange.Broadcast(bLocalIsJumping);
	}
}

float UMockCharacterAbilityComponent::GetBlinkWarmupTimeSeconds() const
{
	return MockCharacterAbilityCVars::BlinkWarmupMS() / 1000.f;
}

float UMockCharacterAbilityComponent::GetStamina() const
{
	if (const FMockCharacterAbilitySyncState* Sync = NetworkPredictionProxy.ReadSyncState<FMockCharacterAbilitySyncState>())
	{
		return Sync->Stamina;
	}
	return 0.f;
}

float UMockCharacterAbilityComponent::GetMaxStamina() const
{
	if (const FMockCharacterAbilityAuxState* Aux = NetworkPredictionProxy.ReadAuxState<FMockCharacterAbilityAuxState>())
	{
		return Aux->MaxStamina;
	}
	return 0.f;
}




void FMockCharacterAbilitySimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockCharacterAbilityBufferTypes>& Input, const TNetSimOutput<TMockCharacterAbilityBufferTypes>& Output)
{
	// WIP NOTES:
	//	-We are creating local copies of the input state that we can mutate and pass on to the parent simulation's tick (LocalCmd, LocalSync, LocalAux). This may seem weird but consider:
	//		-Modifying the actual inputs is a bad idea: inputs are "final" once TNetworkedSimulationModel calls SimulationTick. The sim itself should not modify the inputs after this.
	//		-Tempting to just use the already-allocated Output states and pass them to the parent sim as both Input/Output. But now &Input.Sync == &Output.Sync for example! The parent sim
	//			may write to Output and then later check it against the passed in input (think "did this change"), having no idea that by writing to output it was also writing to input! This seems like it should be avoided.
	//		-It may be worth considering "lazy copying" or allocating of these temp states... but for now that adds more complexity and isn't necessary since the local copies are stack allocated and
	//			these state structures are small and contiguous.
	//
	//	-Still, there are some awkwardness when we need to write to both the local temp state *and* the output state explicitly.
	//	-Stamina changes for example: we want to both change the stamina both for this frame (used by rest of code in this function) AND write it to the output state (so it comes in as input next frame)
	//		-Since this sim "owns" stamina, we could copy it immediately to the output state and use the output state exclusively in this code...
	//		-... but that is inconsistent with state that we don't "own" like Location. Since the parent sim will generate a new Location and we can't pass output to the parent as input
	//			-You would be in a situation where "Teleport modified LocalSync.Location" and "another place modified Output.Sync->Stamina".
	//	
	//
	//	Consider: not using inheritance for simulations. Would that make this less akward? 
	//		-May make Sync state case above less weird since it would breakup stamina/location ("owned" vs "not owned"). Stamina could always use output state and we'd have a local copy of the movement input sync state.
	//		-Aux state is still bad even without inheritance: if you want to read and write to an aux variable at different points in the frames, hard to know where "real" value is at any given time without
	//			writing fragile code. Making code be explicit: "I am writing to the current value this frame AND next frames value" seems ok. Could possibly do everything in a local copy and then see if its changed
	//			at the end up ::SimulationTick and then copy it into output aux. Probably should avoid comparison operator each frame so would need to wrap in some struct that could track a dirty flag... seems complicating.
	//

	const float DeltaSeconds = (float)TimeStep.StepMS / 1000.f;

	// Local copies of the const input that we will pass into the parent sim as input.
	FMockCharacterAbilityInputCmd LocalCmd = *Input.Cmd;
	FMockCharacterAbilitySyncState LocalSync = *Input.Sync;
	FMockCharacterAbilityAuxState LocalAux = *Input.Aux;

	const bool bAlreadyBlinking = Input.Aux->BlinkWarmupLeft > 0;
	const bool bAlreadyDashing = Input.Aux->DashTimeLeft > 0;
	const bool bAllowNewActivations = !bAlreadyBlinking && !bAlreadyDashing;

	// -------------------------------------------------------------------------
	//	Regen
	//	-Applies at the start of frame: you can spend it immediately
	//	-hence we have to write to both the LocalSync's stamina (what the rest of the abilities look at for input) and output (what value will be used as input for next frame)
	// -------------------------------------------------------------------------

	{
		const float NewStamina = FMath::Min<float>(LocalSync.Stamina + (DeltaSeconds * LocalAux.StaminaRegenRate), LocalAux.MaxStamina);
		LocalSync.Stamina = NewStamina;
		Output.Sync->Stamina = NewStamina;
	}

	// -------------------------------------------------------------------------
	//	Blink
	// -------------------------------------------------------------------------

	static float BlinkCost = 25.f;
	const int16 BlinkWarmupMS = MockCharacterAbilityCVars::BlinkWarmupMS();

	auto GetBlinkDestination = [&LocalSync]()
	{
		static float BlinkDist = 1000.f;
		const FVector DestLocation = LocalSync.Location + LocalSync.Rotation.RotateVector(FVector(BlinkDist, 0.f, 0.f));
		return DestLocation;
	};

	const bool bBlinkActivate = (Input.Cmd->bBlinkPressed && Input.Sync->Stamina > BlinkCost && bAllowNewActivations);
	if (bBlinkActivate)
	{
		LocalAux.BlinkWarmupLeft = BlinkWarmupMS;

		// Invoke a cue to telegraph where the blink will land. This is making the assumption the handler wouldn't either want to or be able to derive the blink destination from the current state alone
		// The randomValue being calculated here is purposefully to cause mis prediction of the cue, so that we can demonstrate rollback -> resimulate can do the correction seemlessly

		UE_LOGF(LogNetworkPrediction, Warning, "Invoking FMockCharacterAbilityBlinkActivateCue from sim");
		//Output.CueDispatch.Invoke<FMockCharacterAbilityBlinkActivateCue>(GetBlinkDestination(), FMath::Rand() % 255);
	}

	if (LocalAux.BlinkWarmupLeft > 0)
	{
		const int16 NewBlinkWarmupLeft = FMath::Max<int16>(0, LocalAux.BlinkWarmupLeft - TimeStep.StepMS);
		Output.Aux.Get()->BlinkWarmupLeft = NewBlinkWarmupLeft;

		// While blinking is warming up, we disable movement input and the other actions

		LocalCmd.MovementInput.Set(0.f, 0.f, 0.f);
		LocalCmd.bDashPressed = false;
		LocalCmd.bSprintPressed = false;
		LocalCmd.RotationInput = FRotator::ZeroRotator;
		//LocalSync.Rotation = Input.Sync->Rotation;
		LocalSync.Velocity.Set(0.f, 0.f, 0.f);

		if (NewBlinkWarmupLeft <= 0)
		{
			const FVector DestLocation = GetBlinkDestination();
			AActor* OwningActor = UpdatedComponent->GetOwner();
			check(OwningActor);

			// DrawDebugLine(OwningActor->GetWorld(), Input.Sync->Location, DestLocation, FColor::Red, false);	

			// Its unfortunate teleporting is so complicated. It may make sense for a new movement simulation to define this themselves, 
			// but for this mock one, we will just use the engine's AActor teleport.

			if (OwningActor->TeleportTo(DestLocation, LocalSync.Rotation))
			{
				// Component now has the final location
				const FTransform UpdateComponentTransform = GetUpdateComponentTransform();
				LocalSync.Location = UpdateComponentTransform.GetLocation();

				const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - BlinkCost);
				LocalSync.Stamina = NewStamina;
				Output.Sync->Stamina = NewStamina;

				// Invoke a NetCue for the blink. This is essentially capturing the start/end location so that all receivers of the event
				// get the exact coordinates (maybe overkill in practice but key is that we have data that we want to pass out via an event)

				//UE_LOGF(LogNetworkPrediction, Warning, "Invoking FMockCharacterAbilityBlinkCue from sim. %ls - %ls (LocalSync.Location: %ls) ", *Input.Sync->Location.ToString(), *DestLocation.ToString(), *LocalSync.Location.ToString());
				//UE_VLOG(OwningActor, LogNetworkPrediction, Log, TEXT("Invoking FMockCharacterAbilityBlinkCue from sim. %s - %s (LocalSync.Location: %s) "), *Input.Sync->Location.ToString(), *DestLocation.ToString(), *LocalSync.Location.ToString());

				/*
				switch (MockCharacterAbilityCVars::BlinkCueType()) // Only for dev/testing. Not a normal setup.
				{
				case 0:
					// Skip on purpose
					break;
				case 1:
					Output.CueDispatch.Invoke<FMockCharacterAbilityBlinkCue_Weak>(Input.Sync->Location, DestLocation);
					break;
				case 2:
					Output.CueDispatch.Invoke<FMockCharacterAbilityBlinkCue_ReplicatedNonPredicted>(Input.Sync->Location, DestLocation);
					break;

				case 3:
					Output.CueDispatch.Invoke<FMockCharacterAbilityBlinkCue_ReplicatedXOrPredicted>(Input.Sync->Location, DestLocation);
					break;

				case 4:
					Output.CueDispatch.Invoke<FMockCharacterAbilityBlinkCue_Strong>(Input.Sync->Location, DestLocation);
					break;

				};
				*/				
			}
		}
	}

	// -------------------------------------------------------------------------
	//	Dash
	//	-Stamina consumed on initial press
	//	-MaxSpeed/Acceleration are jacked up
	//	-Dash lasts for 400ms (DashDurationMS)
	//		-Division of frame times can cause you to dash for longer. We would have to break up simulation steps to support this 100% accurately.
	//	-Movement input is synthesized while in dash state. That is, we force forward movement and ignore what was actually fed into the simulation (move input only)
	// -------------------------------------------------------------------------

	static float DashCost = 25.f;
	static int16 DashDurationMS = 500;

	const bool bDashActivate = (Input.Cmd->bDashPressed && Input.Sync->Stamina > DashCost && bAllowNewActivations);
	if (bDashActivate)
	{
		// Start dashing
		LocalAux.DashTimeLeft = DashDurationMS;

		const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - DashCost);
		LocalSync.Stamina = NewStamina;
		Output.Sync->Stamina = NewStamina;
	}

	if (LocalAux.DashTimeLeft > 0)
	{
		const int16 NewDashTimeLeft = FMath::Max<int16>(LocalAux.DashTimeLeft - (int16)(DeltaSeconds * 1000.f), 0);
		Output.Aux.Get()->DashTimeLeft = NewDashTimeLeft;

		LocalAux.MaxSpeed = MockCharacterAbilityCVars::DashMaxSpeed();
		LocalAux.Acceleration = MockCharacterAbilityCVars::DashAcceleration();

		LocalCmd.MovementInput.Set(1.f, 0.f, 0.f);
		LocalCmd.bBlinkPressed = false;
		LocalCmd.bSprintPressed = false;

		Super::SimulationTick(TimeStep, { &LocalCmd, &LocalSync, &LocalAux }, { Output.Sync, Output.Aux, Output.CueDispatch });

		if (NewDashTimeLeft == 0)
		{
			// Stop when dash is over
			Output.Sync->Velocity.Set(0.f, 0.f, 0.f);
		}

		return;
	}


	// -------------------------------------------------------------------------
	//	Sprint
	//	-Requires 15% stamina to start activation
	//	-Drains at 100 stamina/second after that
	//	-Just bumps up max move speed
	//	-Note how this is a transient value: it is fed into the parent sim's input but never makes it to an output state
	// -------------------------------------------------------------------------

	static float SprintBaseCost = 100.f; // 100 sprint/second cost
	static float SprintStartMinPCT = 0.15f; // 15% stamina required to begin sprinting

	bool bIsSprinting = false;
	if (Input.Cmd->bSprintPressed && bAllowNewActivations)
	{
		const float SprintCostThisFrame = SprintBaseCost * DeltaSeconds;

		if (Input.Aux->bIsSprinting)
		{
			// If we are already sprinting, then we keep sprinting as long as we can afford the absolute cost
			bIsSprinting = LocalSync.Stamina > SprintCostThisFrame;
		}
		else
		{
			// To start sprinting, we have to have SprintStartMinPCT stamina
			bIsSprinting = LocalSync.Stamina > (SprintStartMinPCT * LocalAux.MaxStamina);
		}

		if (bIsSprinting)
		{
			LocalAux.MaxSpeed = MockCharacterAbilityCVars::SprintMaxSpeed();

			const float NewStamina = FMath::Max<float>(0.f, LocalSync.Stamina - SprintCostThisFrame);
			LocalSync.Stamina = NewStamina;
			Output.Sync->Stamina = NewStamina;

		}
	}

	// Update the out aux state and call notifies only if sprinting state as actually changed
	if (bIsSprinting != Input.Aux->bIsSprinting)
	{
		Output.Aux.Get()->bIsSprinting = bIsSprinting;
	}

	Super::SimulationTick(TimeStep, { &LocalCmd, &LocalSync, &LocalAux }, { Output.Sync, Output.Aux, Output.CueDispatch });
}
