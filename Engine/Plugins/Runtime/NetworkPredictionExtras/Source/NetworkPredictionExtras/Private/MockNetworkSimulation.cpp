// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetworkSimulation.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Pawn.h"
#include "UObject/UObjectIterator.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionProxyWrite.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "Async/ParallelFor.h"
#include "GameFramework/GameStateBase.h"
#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

DEFINE_LOG_CATEGORY_STATIC(LogMockNetworkSim, Log, All);

namespace MockNetworkSimCVars
{
static int32 UseVLogger = 1;
static FAutoConsoleVariableRef CVarUseVLogger(TEXT("mns.Debug.UseUnrealVLogger"),
	UseVLogger,	TEXT("Use Unreal Visual Logger\n"),	ECVF_Default);

static int32 UseDrawDebug = 1;
static FAutoConsoleVariableRef CVarUseDrawDebug(TEXT("mns.Debug.UseDrawDebug"),
	UseVLogger,	TEXT("Use built in DrawDebug* functions for visual logging\n"), ECVF_Default);

static float DrawDebugDefaultLifeTime = 30.f;
static FAutoConsoleVariableRef CVarDrawDebugDefaultLifeTime(TEXT("mns.Debug.UseDrawDebug.DefaultLifeTime"),
	DrawDebugDefaultLifeTime, TEXT("Use built in DrawDebug* functions for visual logging"), ECVF_Default);

static int32 UseParallelFor = 0;
static FAutoConsoleVariableRef CVarParallelFor(TEXT("mns.ParallelFor"),
	UseParallelFor,	TEXT("Use Parallel For test\n"),	ECVF_Default);

// -------------------------------

static int32 DoLocalInput = 0;
static FAutoConsoleVariableRef CVarDoLocalInput(TEXT("mns.DoLocalInput"),
	DoLocalInput, TEXT("Submit non 0 imput into the mock simulation"), ECVF_Default);

static int32 RandomLocalInput = 0;
static FAutoConsoleVariableRef CVarRandomLocalInput(TEXT("mns.RandomLocalInput"),
	RandomLocalInput, TEXT("When submitting input to mock simulation, use random values (else will submit 1.f)"), ECVF_Default);

static int32 RequestMispredict = 0;
static FAutoConsoleVariableRef CVarRequestMispredict(TEXT("mns.RequestMispredict"),
	RequestMispredict, TEXT("Causes a misprediction by inserting random value into stream on authority side"), ECVF_Default);

static int32 RequestOOBMod = 0;
static FAutoConsoleVariableRef CVarRequestOOBMod(TEXT("mns.RequestOOBMod"),
	RequestOOBMod, TEXT("Causes an OOB modification to sync state on the server (which will result in a correction)"), ECVF_Default);

static int32 BindAutomatically = 1;
static FAutoConsoleVariableRef CVarBindAutomatically(TEXT("mns.BindAutomatically"),
	BindAutomatically, TEXT("Binds local input and mispredict commands to 5 and 6 respectively"), ECVF_Default);

static float MockSimTolerance = SMALL_NUMBER;
static FAutoConsoleVariableRef CVarMockTolerance(TEXT("mns.Tolerance"),
	MockSimTolerance, TEXT("Binds local input and mispredict commands to 5 and 6 respectively"), ECVF_Default);
}

// ============================================================================================

static bool ForceMispredict = false;

// -------------------------------------------------------------------------------------------------------------------------------
//	Networked Simulation Model Def
// -------------------------------------------------------------------------------------------------------------------------------

class FMockNetworkModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = FMockNetworkSimulation;
	using StateTypes = TMockNetworkSimulationBufferTypes;
	using Driver = UMockNetworkSimulationComponent;

	static const TCHAR* GetName() { return TEXT("MockNetSim"); }
};

NP_MODEL_REGISTER(FMockNetworkModelDef);

NETSIMCUE_REGISTER(FMockCue, TEXT("MockCue"));
NETSIMCUESET_REGISTER(UMockNetworkSimulationComponent, FMockCueSet);

// -------------------------------------------------------------------------------------------------------------------------------
//	Simulation
// -------------------------------------------------------------------------------------------------------------------------------

void FMockNetworkSimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockNetworkSimulationBufferTypes>& Input, const TNetSimOutput<TMockNetworkSimulationBufferTypes>& Output)
{
	Output.Sync->Total = Input.Sync->Total + (Input.Cmd->InputValue * Input.Aux->Multiplier * ((float)TimeStep.StepMS / 1000.f));

	if (Input.Cmd->InputValue > 0.f)
	{
		UE_LOGF(LogMockNetworkSim, Warning, "[%d] Output.Sync->Total: %f. Input.Cmd->InputValue: %f... %d", TimeStep.Frame, Output.Sync->Total, Input.Cmd->InputValue, TimeStep.StepMS);
	}

	// Dev hack to force mispredict
	if (ForceMispredict)
	{
		const float Fudge = FMath::FRand() * 100.f;
		Output.Sync->Total += Fudge;
		UE_LOGF(LogMockNetworkSim, Warning, "ForcingMispredict via CVar. Fudge=%.2f. NewTotal: %.2f", Fudge, Output.Sync->Total);
		
		ForceMispredict = false;
	}
	
	if ((int32)(Input.Sync->Total/10.f) < (int32)(Output.Sync->Total/10.f))
	{
		// Emit a cue for crossing this threshold. Note this could mostly be accomplished with a state transition (by detecting this in FinalizeFrame)
		// But to illustrate Cues, we are adding a random number to the cue's payload. The point being cues can transmit data that is not part of the sync/aux state.
		Output.CueDispatch.Invoke<FMockCue>( FMath::Rand() % 1024 );
	}
}

/*
static void Interpolate(const TInterpolatorParameters<FMockSyncState, FMockAuxState>& Params)
{
	Params.Out.Sync.Total = Params.From.Sync.Total + ((Params.To.Sync.Total - Params.From.Sync.Total) * Params.InterpolationPCT);
	Params.Out.Aux = Params.To.Aux;
}*/

// ===============================================================================================
//
// ===============================================================================================

UMockNetworkSimulationComponent::UMockNetworkSimulationComponent()
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;

	bWantsInitializeComponent = true;
	bAutoActivate = true;
	SetIsReplicatedByDefault(true);

	bWantsInitializeComponent = true;

	// Set default bindings. This is not something you would ever want in a shipping game,
	// but the optional, latent, non evasive nature of the NetworkPredictionExtras module makes this ideal approach
	if (HasAnyFlags(RF_ClassDefaultObject) == false && MockNetworkSimCVars::BindAutomatically > 0)
	{
		if (ULocalPlayer* LocalPlayer = GetWorld()->GetFirstLocalPlayerFromController())
		{
			LocalPlayer->Exec(GetWorld(), TEXT("setbind Five \"mns.DoLocalInput 1 | OnRelease mns.DoLocalInput 0\""), *GLog);
			LocalPlayer->Exec(GetWorld(), TEXT("setbind Six \"mns.RequestMispredict 1\""), *GLog);

			LocalPlayer->Exec(GetWorld(), TEXT("setbind Nine nms.Debug.LocallyControlledPawn"), *GLog);
			LocalPlayer->Exec(GetWorld(), TEXT("setbind Zero nms.Debug.ToggleContinous"), *GLog);
		}
	}
}

void UMockNetworkSimulationComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	const ENetRole OwnerRole = GetOwnerRole();

	// Check if we should trip a mispredict. (Note how its not possible to do this inside the Update function!)
	if (OwnerRole == ROLE_Authority && MockNetworkSimCVars::RequestMispredict)
	{
		ForceMispredict = true;
		MockNetworkSimCVars::RequestMispredict = 0;
	}

	if (OwnerRole == ROLE_Authority && MockNetworkSimCVars::RequestOOBMod)
	{
		NetworkPredictionProxy.WriteSyncState<FMockSyncState>([](FMockSyncState& Sync)
		{
			Sync.Total += 5000;
		}, "123");

		MockNetworkSimCVars::RequestOOBMod = 0;
	}
	
	// Mock example of displaying
	DrawDebugString( GetWorld(), GetOwner()->GetActorLocation() + FVector(0.f,0.f,100.f), *LexToString(MockValue), nullptr, FColor::White, 0.00001f );
}

void UMockNetworkSimulationComponent::HandleCue(const FMockCue& MockCue, const FNetSimCueSystemParamemters& SystemParameters)
{
	UE_LOGF(LogMockNetworkSim, Display, "MockCue Handled! Value: %d", MockCue.RandomData);
}

// --------------------------------------------------------------------------------------------------------------

void UMockNetworkSimulationComponent::InitializeNetworkPredictionProxy()
{
	MockNetworkSimulation = MakeUnique<FMockNetworkSimulation>();
	NetworkPredictionProxy.Init<FMockNetworkModelDef>(GetWorld(), GetReplicationProxies(), MockNetworkSimulation.Get(), this);
}

void UMockNetworkSimulationComponent::InitializeSimulationState(FMockSyncState* Sync, FMockAuxState* Aux)
{
	if (Sync)
	{
		Sync->Total = MockValue;
	}
}

void UMockNetworkSimulationComponent::ProduceInput(const int32 DeltaTimeMS, FMockInputCmd* Cmd)
{
	if (MockNetworkSimCVars::DoLocalInput)
	{
		Cmd->InputValue = (MockNetworkSimCVars::RandomLocalInput > 0 ? FMath::FRand() * 10.f : 1.f);
	}
	else
	{
		Cmd->InputValue = 0.f;
	}
}

void UMockNetworkSimulationComponent::FinalizeFrame(const FMockSyncState* Sync, const FMockAuxState* Aux)
{
	npCheckSlow(Sync);
	MockValue = Sync->Total;
}

// ------------------------------------------------------------------------------------------------------------------------
//
// ------------------------------------------------------------------------------------------------------------------------

FAutoConsoleCommandWithWorldAndArgs MockNetworkSimulationSpawnCmd(TEXT("mns.Spawn"), TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& InArgs, UWorld* World) 
{	
	bool bFoundWorld = false;
	int32 NumToSpawn = 1;
	if (InArgs.Num() > 0)
	{
		LexFromString(NumToSpawn, *InArgs[0]);
	}

	for (TObjectIterator<UWorld> It; It; ++It)
	{
		if (It->WorldType == EWorldType::PIE && It->GetNetMode() != NM_Client)
		{
			bFoundWorld = true;
			for (TActorIterator<APawn> ActorIt(*It); ActorIt; ++ActorIt)
			{
				if (APawn* Pawn = *ActorIt)
				{
					
					UMockNetworkSimulationComponent* NewComponent = NewObject<UMockNetworkSimulationComponent>(Pawn);
					NewComponent->RegisterComponent();
				}
			}

			// why not
			AGameStateBase* GameState = It->GetGameState();
			for (int32 i=1; i < NumToSpawn; ++i)
			{
				UMockNetworkSimulationComponent* NewComponent = NewObject<UMockNetworkSimulationComponent>(GameState);
				NewComponent->RegisterComponent();
			}
		}
	}
}));






// ---------------------------------------------------------------------------------------
// Proof of concept for a ParallelFor implementation of simulation tick
//		-(Note: This is completely optional for using NP!)
//		-Specialize TLocalTickService for the FMockNetworkModelDef to tick multiple sims in a ParallelFor
//		-This is mainly a POC for doing custom batch ticking. The ParallelFor itself could be moved to TLocalTickServiceBase and be an option for anyone to use.
//		-Right now still trying to understand if this is beneficial 
// ---------------------------------------------------------------------------------------

#define NP_MOCK_SIM_PARALLELFOR 1
#if NP_MOCK_SIM_PARALLELFOR


template<>
class TLocalTickService<FMockNetworkModelDef> : public TLocalTickServiceBase<FMockNetworkModelDef>
{
public:

	using ModelDef = FMockNetworkModelDef;
	using StateTypes = typename FMockNetworkModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	TLocalTickService(TModelDataStore<FMockNetworkModelDef>* InDataStore) : TLocalTickServiceBase<FMockNetworkModelDef>(InDataStore)
	{

	}

	void RegisterInstance(FNetworkPredictionID ID)
	{
		TLocalTickServiceBase<FMockNetworkModelDef>::RegisterInstance(ID);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		TLocalTickServiceBase<FMockNetworkModelDef>::UnregisterInstance(ID);
	}

	void Tick(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep) final override
	{
		SCOPE_CYCLE_COUNTER(STAT_NetworkPrediction_MockSimTick);

		if (MockNetworkSimCVars::UseParallelFor)
		{
			const int32 InputFrame = ServiceStep.LocalInputFrame;
			const int32 OutputFrame = ServiceStep.LocalOutputFrame;

			const int32 StartTime = Step.TotalSimulationTime;
			const int32 EndTime = ServiceStep.EndTotalSimulationTime;

			TArray<FInstance> InstancesToTickTempArray;
			InstancesToTick.GenerateValueArray(InstancesToTickTempArray);

			ParallelFor(InstancesToTickTempArray.Num(), [&](int32 Index)
			{
				FInstance& InstanceToTick = InstancesToTickTempArray[Index];

				TInstanceData<ModelDef>& Instance = DataStore->Instances.GetByIndexChecked(InstanceToTick.InstanceIdx);
				TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(InstanceToTick.FrameBufferIdx);

				typename TInstanceFrameState<ModelDef>::FFrame& InputFrameData = Frames.Buffer[InputFrame];
				typename TInstanceFrameState<ModelDef>::FFrame& OutputFrameData = Frames.Buffer[OutputFrame];

				UE_NP_TRACE_SIM_TICK(InstanceToTick.TraceID);

				// Copy current input into the output frame. This is redundant in the case where we are polling
				// local input but is needed in the other cases. Simpler to just copy it always.
				if (Instance.NetRole == ROLE_SimulatedProxy)
				{
					OutputFrameData.InputCmd = InputFrameData.InputCmd;
				}

				TTickUtil<ModelDef>::DoTick(Instance, InputFrameData, OutputFrameData, Step, EndTime, GetTickContext<false>(Instance.NetRole));
			});
		}
		else
		{
			TLocalTickServiceBase<FMockNetworkModelDef>::Tick(Step, ServiceStep);
		}
	}

	void TickResim(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep)
	{
		TLocalTickServiceBase<FMockNetworkModelDef>::TickResim(Step, ServiceStep);
	}

	void BeginRollback(const int32 LocalFrame, const int32 StartTimeMS, const int32 ServerFrame)
	{
		TLocalTickServiceBase<FMockNetworkModelDef>::BeginRollback(LocalFrame, StartTimeMS, ServerFrame);
	}
};

#endif
