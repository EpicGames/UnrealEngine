// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockRootMotionComponent.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "NetworkPredictionProxyInit.h"
#include "Components/SkeletalMeshComponent.h"
#include "NetworkPredictionProxyWrite.h"
#include "Animation/AnimInstance.h"
#include "NetworkPredictionReplicatedManager.h"
#include "MockRootMotionSourceObject.h"
#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

DEFINE_LOG_CATEGORY_STATIC(LogMockRootMotionComponent, Log, All);

/** NetworkedSimulation Model type */
class FMockRootMotionModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using StateTypes = MockRootMotionStateTypes;
	using Simulation = FMockRootMotionSimulation;
	using Driver = UMockRootMotionComponent;

	static const TCHAR* GetName() { return TEXT("MockRootMotion"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers + 6; }
};

NP_MODEL_REGISTER(FMockRootMotionModelDef);

// -------------------------------------------------------------------------------------------------------
//	UMockRootMotionComponent
// -------------------------------------------------------------------------------------------------------

void UMockRootMotionComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	Super::SetUpdatedComponent(NewUpdatedComponent);
	FindAndCacheAnimInstance();
}

void UMockRootMotionComponent::FindAndCacheAnimInstance()
{
	if (AnimInstance != nullptr)
	{
		return;
	}

	AActor* Owner = GetOwner();
	npCheckSlow(Owner);

	USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Owner->GetRootComponent());
	if (SkelMeshComp == nullptr)
	{
		SkelMeshComp = Owner->FindComponentByClass<USkeletalMeshComponent>();
	}

	if (SkelMeshComp)
	{
		AnimInstance = SkelMeshComp->GetAnimInstance();
	}
}

void UMockRootMotionComponent::InitializeNetworkPredictionProxy()
{
	if (!npEnsureMsgf(UpdatedComponent != nullptr, TEXT("No UpdatedComponent set on %s. Skipping root motion init."), *GetPathName()))
	{
		return;
	}

	FindAndCacheAnimInstance();

	if (!npEnsureMsgf(AnimInstance != nullptr, TEXT("No AnimInstance set on %s. Skipping root motion init."), *GetPathName()))
	{
		return;
	}

	OwnedMockRootMotionSimulation = MakeUnique<FMockRootMotionSimulation>();
	OwnedMockRootMotionSimulation->SourceStore = this;
	OwnedMockRootMotionSimulation->RootMotionComponent = AnimInstance->GetOwningComponent();
	OwnedMockRootMotionSimulation->SetComponents(UpdatedComponent, UpdatedPrimitive);

	NetworkPredictionProxy.Init<FMockRootMotionModelDef>(GetWorld(), GetReplicationProxies(), OwnedMockRootMotionSimulation.Get(), this);
}

void UMockRootMotionComponent::InitializeSimulationState(FMockRootMotionSyncState* SyncState, FMockRootMotionAuxState* AuxState)
{
	// This assumes no animation is currently playing. Any "play anim on startup" should go through the same path 
	// as playing an animation at runttime wrt NP.

	npCheckSlow(UpdatedComponent);
	npCheckSlow(SyncState);

	SyncState->Location = UpdatedComponent->GetComponentLocation();
	SyncState->Rotation = UpdatedComponent->GetComponentQuat().Rotator();
}

void UMockRootMotionComponent::ProduceInput(const int32 SimTimeMS, FMockRootMotionInputCmd* Cmd)
{
	npCheckSlow(Cmd);
	*Cmd = PendingInputCmd;
}

void UMockRootMotionComponent::RestoreFrame(const FMockRootMotionSyncState* SyncState, const FMockRootMotionAuxState* AuxState)
{
	npCheckSlow(UpdatedComponent);

	// Update component transform
	FTransform Transform(SyncState->Rotation.Quaternion(), SyncState->Location, UpdatedComponent->GetComponentTransform().GetScale3D() );
	UpdatedComponent->SetWorldTransform(Transform, false, nullptr, ETeleportType::TeleportPhysics);
}

void UMockRootMotionComponent::FinalizeFrame(const FMockRootMotionSyncState* SyncState, const FMockRootMotionAuxState* AuxState)
{
	npCheckSlow(AnimInstance);

	RestoreFrame(SyncState, AuxState);

	// Update animation state (pose)
	// This only needs to be done in FinalizeFrame because the pose does not directly affect the simulation

	UMockRootMotionSource* Source = ResolveRootMotionSource(AuxState->Source.GetID(), AuxState->Source.GetParamsDataView());
	if (Source)
	{
		const int32 ElapsedMS = NetworkPredictionProxy.GetTotalSimTimeMS() - AuxState->Source.GetStartMS();
		Source->FinalizePose(ElapsedMS, AnimInstance);
	}
}

UMockRootMotionSource* UMockRootMotionComponent::CreateRootMotionSource(TSubclassOf<UMockRootMotionSource> Source)
{
	if (UClass* ClassPtr = Source.Get())
	{
		return NewObject<UMockRootMotionSource>(this, ClassPtr);
	}
	
	UE_LOGF(LogNetworkPrediction, Warning, "CreateRootMotionSource called with null class");
	return nullptr;
}

void UMockRootMotionComponent::Input_PlayRootMotionSourceByClass(TSubclassOf<UMockRootMotionSource> Source)
{
	if (UMockRootMotionSource* CDO = Source.GetDefaultObject())
	{
		Input_PlayRootMotionSource(CDO);
	}
	else
	{
		UE_LOGF(LogNetworkPrediction, Warning, "Input_PlayRootMotionSourceByClass called with null class");
	}
}

void UMockRootMotionComponent::Input_PlayRootMotionSource(UMockRootMotionSource* Source)
{
	if (!Source || !Source->IsValidRootMotionSource())
	{
		UE_LOGF(LogNetworkPrediction, Warning, "Invalid Source (%ls) in call to Input_PlayRootMotionSource", *GetNameSafe(Source));
		return;
	}

	UNetworkPredictionWorldManager* NetworkPredictionWorldManager = GetWorld()->GetSubsystem<UNetworkPredictionWorldManager>();
	npCheckSlow(NetworkPredictionWorldManager);

	ANetworkPredictionReplicatedManager* Manager = NetworkPredictionWorldManager->ReplicatedManager;
	if (!npEnsureMsgf(Manager, TEXT("ANetworkPredictionReplicatedManager not available."))) 
	{
		return;
	}

	const int32 ID = Manager->GetIDForObject(Source->GetClass());
	StoreRootMotionSource(ID, Source);

	// Write the new source we are trying to play into the PendingInputCmd
	const int32& SimTimeMS = NetworkPredictionProxy.GetTotalSimTimeMS();
	PendingInputCmd.PlaySource.WriteSource(ID, SimTimeMS);
	
	// If this is instanced, give it a chance to write parameters
	if (Source->HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		PendingInputCmd.PlaySource.WriteParams([Source](FBitWriter& Writer)
		{
			Source->SerializePayloadParameters(Writer);
		});
	}
}
	
void UMockRootMotionComponent::PlayRootMotionSource(UMockRootMotionSource* Source)
{
	if (!Source || !Source->IsValidRootMotionSource())
	{
		UE_LOGF(LogNetworkPrediction, Warning, "Invalid Source (%ls) in call to Input_PlayRootMotionSource", *GetNameSafe(Source));
		return;
	}

	UNetworkPredictionWorldManager* NetworkPredictionWorldManager = GetWorld()->GetSubsystem<UNetworkPredictionWorldManager>();
	npCheckSlow(NetworkPredictionWorldManager);

	ANetworkPredictionReplicatedManager* Manager = NetworkPredictionWorldManager->ReplicatedManager;
	if (!npEnsureMsgf(Manager, TEXT("ANetworkPredictionReplicatedManager not available."))) 
	{
		return;
	}

	// Lookup ID for this class and store it locally
	const int32 ID = Manager->GetIDForObject(Source->GetClass());
	StoreRootMotionSource(ID, Source);

	const int32& SimTimeMS = NetworkPredictionProxy.GetTotalSimTimeMS();
	
	// Write new source directly to the aux state
	NetworkPredictionProxy.WriteAuxState<FMockRootMotionAuxState>([Source, ID, SimTimeMS](FMockRootMotionAuxState& Aux)
	{
		Aux.Source.WriteSource(ID, SimTimeMS);

		// If this is instanced, give it a chance to write parameters
		if (Source->HasAnyFlags(RF_ClassDefaultObject) == false)
		{
			Aux.Source.WriteParams([Source](FBitWriter& Writer)
			{
				Source->SerializePayloadParameters(Writer);
			});
		}

	}, "UMockRootMotionComponent::PlayRootMotionSource");	// The string here is for Insights tracing. "Who did this?"
}

void UMockRootMotionComponent::PlayRootMotionSourceByClass(TSubclassOf<UMockRootMotionSource> Source)
{
	if (UMockRootMotionSource* CDO = Source.GetDefaultObject())
	{
		PlayRootMotionSource(CDO);
	}
	else
	{
		UE_LOGF(LogNetworkPrediction, Warning, "PlayRootMotionSourceByClass called with null class");
	}
}

UMockRootMotionSource* UMockRootMotionComponent::ResolveRootMotionSource(int32 ID, const TArrayView<const uint8>& Data)
{
	if (RootMotionSourceCache.ClassID == ID)
	{
		return RootMotionSourceCache.Instance;
	}

	UNetworkPredictionWorldManager* NetworkPredictionWorldManager = GetWorld()->GetSubsystem<UNetworkPredictionWorldManager>();
	npCheckSlow(NetworkPredictionWorldManager);

	ANetworkPredictionReplicatedManager* Manager = NetworkPredictionWorldManager->ReplicatedManager;
	if (!npEnsureMsgf(Manager, TEXT("ANetworkPredictionReplicatedManager not available."))) 
	{
		return nullptr;
	}

	RootMotionSourceCache.ClassID = ID;
	RootMotionSourceCache.Instance = nullptr;
	
	TSoftObjectPtr<UObject> SoftObjPtr = Manager->GetObjectForID(ID);
	UObject* Obj = SoftObjPtr.Get();
	if (Obj == nullptr)
	{
		return nullptr;
	}

	UClass* ClassObj = Cast<UClass>(Obj);
	if (ClassObj == nullptr || !ClassObj->IsChildOf(UMockRootMotionSource::StaticClass()))
	{
		UE_LOGF(LogNetworkPrediction, Warning, "Resolved object %ls is not a subclass of URootMotionSource", *GetNameSafe(Obj));
		return nullptr;
	}

	if (Data.Num() == 0)
	{
		// No parameters: this implies no instantiation/CDO 
		// (are we sure? Should there be a virtual here? What if we have a no parameter but internal state having source?)
		RootMotionSourceCache.Instance = Cast<UMockRootMotionSource>(ClassObj->GetDefaultObject(false));
	}
	else
	{
		RootMotionSourceCache.Instance = NewObject<UMockRootMotionSource>(this, ClassObj);
		
		FBitReader BitReader(const_cast<uint8*>(Data.GetData()), (int64)Data.Num() << 3);
		RootMotionSourceCache.Instance->SerializePayloadParameters(BitReader);
	}	

	npCheckSlow(RootMotionSourceCache.Instance);
	return RootMotionSourceCache.Instance;
}

void UMockRootMotionComponent::StoreRootMotionSource(int32 ID, UMockRootMotionSource* Source)
{
	RootMotionSourceCache.ClassID = ID;
	RootMotionSourceCache.Instance = Source;
}
