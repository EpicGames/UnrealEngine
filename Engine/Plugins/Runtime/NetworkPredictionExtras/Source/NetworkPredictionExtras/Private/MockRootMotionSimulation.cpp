// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockRootMotionSimulation.h"
#include "MockRootMotionSourceObject.h"
#include "NetworkPredictionTickState.h"

DEFINE_LOG_CATEGORY_STATIC(LogMockRootMotion, Log, All);

// Core tick function for updating root motion.
//	-Handle InputCmd: possibly turn input state into a playing RootMotion source
//	-If RootMotion source is playing, call into UMockRootMotionSource to evaluate (get the delta transform and update the output Sync state)
//	-Use delta transform to actually sweep move our collision through the world

void FMockRootMotionSimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<MockRootMotionStateTypes>& Input, const TNetSimOutput<MockRootMotionStateTypes>& Output)
{
	npCheckSlow(UpdatedComponent);
	npCheckSlow(RootMotionComponent);
	npCheckSlow(SourceStore);

	FMockRootMotionSyncState LocalSync = *Input.Sync;
	const FMockRootMotionAuxState* LocalAuxPtr = Input.Aux;

	// See if InputCmd wants to play a new RootMotion (only if we aren't currently playing one)
	if (Input.Cmd->PlaySource.GetID() != LocalAuxPtr->Source.GetID() && Input.Cmd->PlaySource.GetStartMS() > LocalAuxPtr->EndTimeMS)
	{	
		// NOTE: This is not a typical gameplay setup. This code essentially allows the client to 
		// start new RootMotionSources anytime it wants to.
		//
		// A real game is going to have higher level logic. Like "InputCmd wants to activate an ability, can we do that?"
		// and if so; "update Sync State to reflect the new animation".	

		// Copy the play parameters to the aux state
		FMockRootMotionAuxState* OutAux = Output.Aux.Get();
		OutAux->Source = Input.Cmd->PlaySource;
		LocalAuxPtr = OutAux;

		// Question: should we advance the root motion here or not? When you play a new montage, do you expect the next render 
		// frame to be @ t=0? Or should we advance it by TimeStep.StepMS? 
		//
		// It seems one frame of no movement would be bad. If we were chaining animations together, we wouldn't want to enforce
		// a system one stationary frame (which will depend on TimeStep.MS!)
	}

	// Copy input to output
	*Output.Sync = LocalSync;

	if (LocalAuxPtr->Source.IsValid() == false)
	{
		// We are aren't playing a root motion source so nothing left to do
		return;
	}

	UMockRootMotionSource* Source = this->ResolveRootMotionSource(LocalAuxPtr->Source);
	if (!npEnsureMsgf(Source, TEXT("Could not resolve root motion source %d"), LocalAuxPtr->Source.GetID()))
	{
		// We think we are playing a root motion source but could not resolve it to a UMockRootMotionSource
		return;
	}

	// Call into root motion source map to actually update the root motino state
	const int32 ElapsedMS = TimeStep.TotalSimulationTime - LocalAuxPtr->Source.GetStartMS();
	FMockRootMotionReturnValue Result = Source->Step(FMockRootMotionStepParameters{LocalSync.Location, LocalSync.Rotation, ElapsedMS, TimeStep.StepMS});

	// StepRootMotion should return local delta transform, we need to convert to world
	FTransform DeltaWorldTransform;

	if (Result.TransformType == FMockRootMotionReturnValue::ETransformType::AnimationRelative)
	{
		// Calculate new actor transform after applying root motion to this component
		// this was lifted from USkeletalMeshComponent::ConvertLocalRootMotionToWorld
		const FTransform ActorToWorld = RootMotionComponent->GetOwner()->GetTransform();

		const FTransform ComponentToActor = ActorToWorld.GetRelativeTransform(RootMotionComponent->GetComponentTransform());
		const FTransform NewComponentToWorld = Result.DeltaTransform * RootMotionComponent->GetComponentTransform();
		const FTransform NewActorTransform = ComponentToActor * NewComponentToWorld;

		const FVector DeltaWorldTranslation = NewActorTransform.GetTranslation() - ActorToWorld.GetTranslation();

		const FQuat NewWorldRotation = RootMotionComponent->GetComponentTransform().GetRotation() * Result.DeltaTransform.GetRotation();
		const FQuat DeltaWorldRotation = NewWorldRotation * RootMotionComponent->GetComponentTransform().GetRotation().Inverse();

		DeltaWorldTransform = FTransform(DeltaWorldRotation, DeltaWorldTranslation);

		/*
		UE_LOGF(LogRootMotion, Log,  "ConvertLocalRootMotionToWorld LocalT: %ls, LocalR: %ls, WorldT: %ls, WorldR: %ls.",
			*InTransform.GetTranslation().ToCompactString(), *InTransform.GetRotation().Rotator().ToCompactString(),
			*DeltaWorldTransform.GetTranslation().ToCompactString(), *DeltaWorldTransform.GetRotation().Rotator().ToCompactString());
			*/		
	}
	else
	{
		DeltaWorldTransform = Result.DeltaTransform;
	}


	if (Result.State == FMockRootMotionReturnValue::ESourceState::Stop)
	{
		// Done with this source
		FMockRootMotionAuxState* OutAux = Output.Aux.Get();
		npCheckSlow(OutAux);
		OutAux->Source.Reset();
		OutAux->EndTimeMS = TimeStep.TotalSimulationTime;
	}

	// ---------------------------------------------------------------------
	// Move the component via collision sweep
	//	-This could be better: to much converting between FTransforms, Rotators, quats, etc.
	//	-Problem of "movement can be blocked but rotation can't". Can be unclear exactly what to do 
	//		(should block in movement cause a block in rotation?)
	// ---------------------------------------------------------------------
	FQuat NewRotation(DeltaWorldTransform.Rotator() + LocalSync.Rotation);

	// Actually do the sweep
	FHitResult HitResult;
	SafeMoveUpdatedComponent(DeltaWorldTransform.GetTranslation(), NewRotation, true, HitResult, ETeleportType::TeleportPhysics);

	// The component was actually moved, so pull transform back out
	FTransform EndTransform = UpdatedComponent->GetComponentTransform(); 
	Output.Sync->Location = EndTransform.GetTranslation();
	Output.Sync->Rotation = EndTransform.GetRotation().Rotator();
}
