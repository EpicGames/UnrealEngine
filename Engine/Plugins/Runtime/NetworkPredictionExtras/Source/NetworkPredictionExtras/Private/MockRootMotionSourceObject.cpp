// Copyright Epic Games, Inc. All Rights Reserved

#include "MockRootMotionSourceObject.h"
#include "Engine/ObjectLibrary.h"
#include "NetworkPredictionReplicatedManager.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "Curves/CurveVector.h"

UMockRootMotionSourceClassMap* UMockRootMotionSourceClassMap::Singleton = nullptr;

void UMockRootMotionSourceClassMap::BeginDestroy()
{
	Super::BeginDestroy();

	if (this == Singleton)
	{
		this->RemoveFromRoot();
		Singleton = nullptr;
	}

	if (AuthoritySpawnCallbackHandle.IsValid())
	{
		ANetworkPredictionReplicatedManager::UnregisterOnAuthoritySpawn(AuthoritySpawnCallbackHandle);
		AuthoritySpawnCallbackHandle.Reset();
	}
}

void UMockRootMotionSourceClassMap::InitSingleton()
{
	Singleton = NewObject<UMockRootMotionSourceClassMap>(GetTransientPackage());
	Singleton->AddToRoot();

	Singleton->AuthoritySpawnCallbackHandle = ANetworkPredictionReplicatedManager::OnAuthoritySpawn([](ANetworkPredictionReplicatedManager* Manager)
	{
		UMockRootMotionSourceClassMap::Get()->CopyToReplicatedManager(Manager);
	});

	BuildClassMap();
}

void UMockRootMotionSourceClassMap::BuildClassMap()
{
	npCheck(Singleton);
	Singleton->BuildClassMap_Internal();
}

void UMockRootMotionSourceClassMap::BuildClassMap_Internal()
{
	// --------------------------------------------------------
	//	Native
	// --------------------------------------------------------

	TArray<UClass*> Classes;
	GetDerivedClasses(UMockRootMotionSource::StaticClass(), Classes, true);

	//UE_LOGF(LogTemp, Warning, "UMockRootMotionSourceClassMap::BuildClassMap. Found %d classes.", Classes.Num());

	SourceList.Reset();
	LookupMap.Reset();

	for (UClass* FoundClass : Classes)
	{
		FString Str = FoundClass->GetName();
		if (Str.StartsWith("SKEL_") || Str.StartsWith("REINST_"))
		{
			continue;
		}

		TSubclassOf<UMockRootMotionSource> AsSubclass(FoundClass);
		if (npEnsureMsgfSlow(AsSubclass.Get(), TEXT("%s not valid subclass?"), *FoundClass->GetName()))
		{
			SourceList.Add(AsSubclass.Get());
		}
	}

	// --------------------------------------------------------
	//	Blueprint
	// --------------------------------------------------------
	
	TArray<FAssetData> AssetDataList;

	//UE_LOGF(LogTemp, Warning, "Scanning Asset Registry for root motion source objects");

	UObjectLibrary* Lib = UObjectLibrary::CreateLibrary(UMockRootMotionSource::StaticClass(), true, true);

	Lib->LoadBlueprintAssetDataFromPath(TEXT("/Game"));
	Lib->GetAssetDataList(AssetDataList);
	//UE_LOGF(LogTemp, Warning, "Found %d items in AssetDataList", AssetDataList.Num());

	AssetDataList.Reset();
	Lib->LoadBlueprintAssetDataFromPath(TEXT("/NetworkPredictionExtras"));
	Lib->GetAssetDataList(AssetDataList);

	for (FAssetData& AssetData : AssetDataList)
	{
		FString ClassPathString = AssetData.GetObjectPathString() + TEXT("_c");
		FSoftObjectPath SoftObjPath(ClassPathString);
		SourceList.Add(TSoftClassPtr<UMockRootMotionSource>(SoftObjPath));
	}

	// --------------------------------------------------------
	//	Sort
	// --------------------------------------------------------

	Algo::SortBy(SourceList, [](const TSoftClassPtr<UMockRootMotionSource>& Item){ return Item.ToString(); });
	
	for (auto It = SourceList.CreateIterator(); It; ++It)
	{
		TSoftClassPtr<UMockRootMotionSource>& Subclass = *It;
		UClass* LoadedClass = Subclass.Get();
		if (LoadedClass)
		{
			LookupMap.Add(LoadedClass, It.GetIndex());
		}

		//UE_LOGF(LogTemp, Warning, "%ls = %d (%ls)", *Subclass.ToString(), It.GetIndex(), LoadedClass ? TEXT("Loaded") : TEXT("Unloaded"));
	}

	// Verify lookup table (note that unloaded entries won't be in LookupMap
	for (auto& MapIt : LookupMap)
	{
		const int32 idx = MapIt.Value;
		npEnsure(SourceList.IsValidIndex(idx) && SourceList[idx].Get() == MapIt.Key);
	}
}

void UMockRootMotionSourceClassMap::CopyToReplicatedManager(ANetworkPredictionReplicatedManager* Manager)
{
	npCheckSlow(Manager);

	for (TSoftClassPtr<UMockRootMotionSource>& Subclass : SourceList)
	{
		Manager->AddObjectToSharedPackageMap(TSoftObjectPtr<UObject>(Subclass.ToSoftObjectPath()));
	}
}


// --------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------

FMockRootMotionReturnValue UMockRootMotionSource_Montage::Step(const FMockRootMotionStepParameters& Parameters)
{
	npCheckSlow(this->Montage);

	const float MinPosition = 0.f;
	const float MaxPosition = Montage->GetPlayLength();
	
	const float DeltaPlayPosition = ((float)Parameters.DeltaMS / 1000.f) * PlayRate;
	const float StartPlayPosition = FMath::Clamp(((float)Parameters.ElapsedMS / 1000.f) * PlayRate, MinPosition, MaxPosition);
	const float EndPlayPosition =  FMath::Clamp(StartPlayPosition + DeltaPlayPosition, MinPosition, MaxPosition);

	FMockRootMotionReturnValue ReturnValue;
	
	if (EndPlayPosition >= MaxPosition)
	{
		ReturnValue.Stop();
	}

	// Extract root motion from animation sequence 
	ReturnValue.DeltaTransform = Montage->ExtractRootMotionFromTrackRange(StartPlayPosition, EndPlayPosition, FAnimExtractContext());
	ReturnValue.TransformType = FMockRootMotionReturnValue::ETransformType::AnimationRelative;

	//UE_LOGF(LogTemp, Warning, "%.4f: Delta: %ls", EndPosition, *NewTransform.GetTranslation().ToString());
	return ReturnValue;
}

void UMockRootMotionSource_Montage::FinalizePose(int32 ElapsedMS, UAnimInstance* AnimInstance) const
{
	npCheckSlow(AnimInstance);
	npCheckSlow(this->Montage);

	if (!AnimInstance->Montage_IsPlaying(Montage))
	{
		AnimInstance->Montage_Play(Montage);
	}

	const float PlayPosition = ((float)ElapsedMS / 1000.f) * PlayRate;
	AnimInstance->Montage_SetPosition(Montage, PlayPosition);
}

// ---------------------------------------------

FMockRootMotionReturnValue UMockRootMotionSource_Curve::Step(const FMockRootMotionStepParameters& Parameters)
{
	npCheckSlow(this->Curve);

	FMockRootMotionReturnValue ReturnValue;

	float MinPosition = 0.f;
	float MaxPosition = 0.f;
	Curve->GetTimeRange(MinPosition, MaxPosition);

	const float DeltaPlayPosition = ((float)Parameters.DeltaMS / 1000.f) * PlayRate;
	const float StartPlayPosition = ((float)Parameters.ElapsedMS / 1000.f) * PlayRate;
	const float EndPlayPosition =  FMath::Clamp(StartPlayPosition + DeltaPlayPosition, MinPosition, MaxPosition);

	if (EndPlayPosition >= MaxPosition)
	{
		ReturnValue.Stop();
	}

	const FVector Start = Curve->GetVectorValue(StartPlayPosition);
	const FVector End = Curve->GetVectorValue(EndPlayPosition);

	const FVector DeltaTranslation = (End - Start) * this->TranslationScale;	
	
	ReturnValue.DeltaTransform.SetTranslation(DeltaTranslation);

	return ReturnValue; 
}

// ---------------------------------------------

FMockRootMotionReturnValue UMockRootMotionSource_MoveToLocation::Step(const FMockRootMotionStepParameters& Parameters)
{
	npEnsureMsgf(this->bDestinationSet, TEXT("UMockRootMotionSource_MoveToLocation::Step called without bDestinationSet"));

	FMockRootMotionReturnValue ReturnValue;

	const FVector DeltaToDestination = this->Destination - Parameters.Location;
	const float DistRemainingSq = DeltaToDestination.SizeSquared();

	const float StepDistance = this->Velocity * ((float)Parameters.DeltaMS / 1000.f);

	if (DistRemainingSq <= StepDistance * StepDistance)
	{
		ReturnValue.DeltaTransform.SetTranslation(DeltaToDestination);
		ReturnValue.Stop();
	}
	else
	{
		ReturnValue.DeltaTransform.SetTranslation(DeltaToDestination.GetUnsafeNormal() * StepDistance);
	}
	
	return ReturnValue;
}
