// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionExtrasCharacter.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/InputComponent.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/SpringArmComponent.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "MockCharacterAbilitySimulation.h"
#include "NetworkPredictionLog.h"

#include "HAL/PlatformStackWalk.h"
#include "HAL/ThreadHeartBeat.h"

namespace CharacterCVars
{

int32 CameraStyle = 0;
FAutoConsoleVariableRef CVarPenetrationPullbackDistance(TEXT("NetworkPredictionExtras.Character.CameraSyle"),
	CameraStyle,
	TEXT("Sets camera mode style in ANetworkPredictionExtrasCharacter \n")
	TEXT("0=camera fixed behind pawn. 1-3 are variations of a free camera system (Gamepad recommended)."),
	ECVF_Default);

static int32 BindAutomatically = 1;
static FAutoConsoleVariableRef CVarBindAutomatically(TEXT("NetworkPredictionExtras.Character.BindAutomatically"),
	BindAutomatically, TEXT("Binds local input and mispredict commands to 5 and 6 respectively"), ECVF_Default);
}

const FName Name_CharacterMotionComponent(TEXT("CharacterMotionComponent"));

ANetworkPredictionExtrasCharacter::ANetworkPredictionExtrasCharacter(const FObjectInitializer& ObjectInitializer)
{
	CharacterMotionComponent = CreateDefaultSubobject<UCharacterMotionComponent>(Name_CharacterMotionComponent);
	ensure(CharacterMotionComponent);

	SetReplicateMovement(false);
}

void ANetworkPredictionExtrasCharacter::BeginPlay()
{
	Super::BeginPlay();

	if (UWorld* World = GetWorld())
	{
		if (ensure(CharacterMotionComponent))
		{
			CharacterMotionComponent->ProduceInputDelegate.BindUObject(this, &ANetworkPredictionExtrasCharacter::ProduceInput);
		}

		// Binds 0 and 9 to the debug hud commands. This is just a convenience for the extras plugin. Real projects should bind this themselves
		if (CharacterCVars::BindAutomatically > 0)
		{
			if (ULocalPlayer* LocalPlayer = World->GetFirstLocalPlayerFromController())
			{
				LocalPlayer->Exec(World, TEXT("setbind Nine nms.Debug.LocallyControlledPawn"), *GLog);
				LocalPlayer->Exec(World, TEXT("setbind Zero nms.Debug.ToggleContinous"), *GLog);
			}
		}
	}
}

UNetConnection* ANetworkPredictionExtrasCharacter::GetNetConnection() const
{
	UNetConnection* SuperNetConnection = Super::GetNetConnection();
	if (SuperNetConnection)
	{
		return SuperNetConnection;
	}

	if (bFakeAutonomousProxy)
	{
		if (GetLocalRole() == ROLE_Authority)
		{
			for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PC = Iterator->Get();
				if (PC->GetNetConnection() && PC->GetPawn())
				{
					return PC->GetNetConnection();
				}
			}
		}
		if (GetLocalRole() == ROLE_AutonomousProxy && GetWorld()->GetFirstPlayerController() && GetWorld()->GetFirstPlayerController()->GetPawn())
		{
			return GetWorld()->GetFirstPlayerController()->GetNetConnection();
		}
	}

	return nullptr;
}

void ANetworkPredictionExtrasCharacter::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	// Setup some bindings.
	//
	// This also is not necessary. This would usually be defined in DefaultInput.ini.
	// But the nature of NetworkPredictionExtras is to not assume or require anything of the project its being run in.
	// So in order to make this nice and self contained, manually do input bindings here.
	if (APlayerController* PC = Cast<APlayerController>(GetController()))
	{
		if (PC->PlayerInput)
		{
			//Gamepad
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveRight"), EKeys::Gamepad_LeftX));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveForward"), EKeys::Gamepad_LeftY));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LookYaw"), EKeys::Gamepad_RightX));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LookPitch"), EKeys::Gamepad_RightY));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LeftTriggerAxis"), EKeys::Gamepad_LeftTriggerAxis));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("RightTriggerAxis"), EKeys::Gamepad_RightTriggerAxis));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("LeftShoulder"), EKeys::Gamepad_LeftShoulder));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("RightShoulder"), EKeys::Gamepad_RightShoulder));

			// Keyboard
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveRight"), EKeys::D, 1.f));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveRight"), EKeys::A, -1.f));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveForward"), EKeys::W, 1.f));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("MoveForward"), EKeys::S, -1.f));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LeftTriggerAxis"), EKeys::C, 1.f));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("RightTriggerAxis"), EKeys::LeftControl, 1.f));

			// Mouse
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LookYaw"), EKeys::MouseX));
			PC->PlayerInput->AddAxisMapping(FInputAxisKeyMapping(TEXT("LookPitch"), EKeys::MouseY));
		}
	}

	PlayerInputComponent->BindAxis(TEXT("MoveForward"), this,	&ThisClass::InputAxis_MoveForward);
	PlayerInputComponent->BindAxis(TEXT("MoveRight"), this,		&ThisClass::InputAxis_MoveRight);
	PlayerInputComponent->BindAxis(TEXT("LookYaw"), this,		&ThisClass::InputAxis_LookYaw);
	PlayerInputComponent->BindAxis(TEXT("LookPitch"), this,		&ThisClass::InputAxis_LookPitch);
	PlayerInputComponent->BindAxis(TEXT("RightTriggerAxis"), this,		&ThisClass::InputAxis_MoveUp);
	PlayerInputComponent->BindAxis(TEXT("LeftTriggerAxis"), this,		&ThisClass::InputAxis_MoveDown);

	PlayerInputComponent->BindAction(TEXT("LeftShoulder"), IE_Pressed, this, &ThisClass::Action_LeftShoulder_Pressed);
	PlayerInputComponent->BindAction(TEXT("LeftShoulder"), IE_Released, this, &ThisClass::Action_LeftShoulder_Released);
	PlayerInputComponent->BindAction(TEXT("RightShoulder"), IE_Pressed, this, &ThisClass::Action_RightShoulder_Pressed);
	PlayerInputComponent->BindAction(TEXT("RightShoulder"), IE_Released, this, &ThisClass::Action_RightShoulder_Released);
}

void ANetworkPredictionExtrasCharacter::InputAxis_MoveForward(float Value)
{
	CachedMoveInput.X = FMath::Clamp(Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::InputAxis_MoveRight(float Value)
{
	CachedMoveInput.Y = FMath::Clamp(Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::InputAxis_LookYaw(float Value)
{
	CachedLookInput.X = FMath::Clamp(Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::InputAxis_LookPitch(float Value)
{
	CachedLookInput.Y = FMath::Clamp(Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::InputAxis_MoveUp(float Value)
{
	CachedMoveInput.Z = FMath::Clamp(CachedMoveInput.Z + Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::InputAxis_MoveDown(float Value)
{
	CachedMoveInput.Z = FMath::Clamp(CachedMoveInput.Z - Value, -1.0f, 1.0f);
}

void ANetworkPredictionExtrasCharacter::Tick( float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
	// Do whatever you want here. By now we have the latest movement state and latest input processed.

	if (bFakeAutonomousProxy && GetLocalRole() == ROLE_Authority)
	{
		if (GetRemoteRole() != ROLE_AutonomousProxy)
		{
			SetAutonomousProxy(true);
		}
	}

	// ForceNetUpdate(); // DNC
}

void ANetworkPredictionExtrasCharacter::PrintDebug()
{
	UE_LOGF(LogNetworkPrediction, Warning, "======== ANetworkPredictionExtrasCharacter::PrintDebug ========");

	FSlowHeartBeatScope SuspendHeartBeat;
	FDisableHitchDetectorScope SuspendGameThreadHitch;

	PrintScriptCallstack();

	const SIZE_T StackTraceSize = 65535;
	ANSICHAR* StackTrace = (ANSICHAR*)FMemory::SystemMalloc(StackTraceSize);
	if (StackTrace != nullptr)
	{
		StackTrace[0] = 0;
		// Walk the stack and dump it to the allocated memory.
		FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 1);
		UE_LOGF(LogNetworkPrediction, Log, "Call Stack:\n%ls", ANSI_TO_TCHAR(StackTrace));
		FMemory::SystemFree(StackTrace);
	}
}

float ANetworkPredictionExtrasCharacter::GetMaxMoveSpeed() const
{
	return CharacterMotionComponent->GetMaxMoveSpeed();
}

void ANetworkPredictionExtrasCharacter::SetMaxMoveSpeed(float NewMaxMoveSpeed)
{
	CharacterMotionComponent->SetMaxMoveSpeed(NewMaxMoveSpeed);
}

void ANetworkPredictionExtrasCharacter::AddMaxMoveSpeed(float AdditiveMaxMoveSpeed)
{
	CharacterMotionComponent->AddMaxMoveSpeed(AdditiveMaxMoveSpeed);
}

void ANetworkPredictionExtrasCharacter::ProduceInput(const int32 DeltaMS, FCharacterMotionInputCmd& Cmd)
{
	// Generate user commands. Called right before the Character movement simulation will tick (for a locally controlled pawn)
	// This isn't meant to be the best way of doing a camera system. It is just meant to show a couple of ways it may be done
	// and to make sure we can keep distinct the movement, rotation, and view angles.
	// Change with CVar NetworkPredictionExtras.Character.CameraSyle. Styles 1-3 are really meant to be used with a gamepad.
	//
	// Its worth calling out: the code that happens here is happening *outside* of the Character movement simulation. All we are doing
	// is generating the input being fed into that simulation. That said, this means that A) the code below does not run on the server
	// (and non controlling clients) and B) the code is not rerun during reconcile/resimulates. Use this information guide any
	// decisions about where something should go (such as aim assist, lock on targeting systems, etc): it is hard to give absolute
	// answers and will depend on the game and its specific needs. In general, at this time, I'd recommend aim assist and lock on 
	// targeting systems to happen /outside/ of the system, i.e, here. But I can think of scenarios where that may not be ideal too.

	if (InputPreset == ENetworkPredictionExtrasCharacterInputPreset::Forward)
	{
		Cmd.MovementInput = FVector(1.f, 0.f, 0.f);
		Cmd.RotationInput = FRotator::ZeroRotator;
		return;
	}


	if (GetController() == nullptr)
	{
		if (GetLocalRole() == ENetRole::ROLE_Authority && GetRemoteRole() == ENetRole::ROLE_SimulatedProxy)
		{
			// If we get here, that means this pawn is not currently possessed and we're choosing to provide default do-nothing input
			Cmd = FCharacterMotionInputCmd();
		}

		CachedMoveInput = FVector::ZeroVector;
		CachedLookInput = FVector2D::ZeroVector;

		// We don't have a local controller so we can't run the code below. This is ok. Simulated proxies will just use previous input when extrapolating
		return;
	}
	
	if (USpringArmComponent* SpringComp = FindComponentByClass<USpringArmComponent>())
	{
		// This is not best practice: do not search for component every frame
		SpringComp->bUsePawnControlRotation = true;
	}

	// Simple input scaling. A real game will probably map this to an acceleration curve
	static float LookRateYaw = 150.f;
	static float LookRatePitch = 150.f;

	static float ControllerLookRateYaw = 5.0f;
	static float ControllerLookRatePitch = 5.0;

	// Zero out input structs in case each path doesnt set each member. This is really all we are filling out here.
	Cmd.MovementInput = FVector::ZeroVector;
	Cmd.RotationInput = FRotator::ZeroRotator;

	const float DeltaTimeSeconds = (float)DeltaMS / 1000.f;
	const FVector2D LookInput = CachedLookInput;
	const FVector MoveInput = CachedMoveInput;

	switch (CharacterCVars::CameraStyle)
	{
		case 0:
		{
			// Fixed camera
			if (USpringArmComponent* SpringComp = FindComponentByClass<USpringArmComponent>())
			{
				// Only this camera mode has to set this to false
				SpringComp->bUsePawnControlRotation = false;
			}

			Cmd.RotationInput.Yaw = LookInput.X * LookRateYaw;
			Cmd.RotationInput.Pitch = -LookInput.Y * LookRatePitch;
			Cmd.RotationInput.Roll = 0;
					
			Cmd.MovementInput = MoveInput;
			break;
		}
		case 1:
		{
			// Free camera Restricted 2D movement on XY plane.
			APlayerController* PC = Cast<APlayerController>(GetController());
			if (ensure(PC)) // Requires player controller for now
			{
				// Camera yaw rotation
				PC->AddYawInput(LookInput.X * ControllerLookRateYaw );
				PC->AddPitchInput(-LookInput.Y * ControllerLookRatePitch );

				const float RotationMagMin = (1e-3f);
						
				float RotationInputYaw = 0.f;
				if (MoveInput.Size() >= RotationMagMin)
				{
					FVector DesiredMovementDir = PC->GetControlRotation().RotateVector(MoveInput);

					// 2D xy movement, relative to camera
					{
						FVector DesiredMoveDir2D = FVector(DesiredMovementDir.X, DesiredMovementDir.Y, 0.f);
						DesiredMoveDir2D.Normalize();

						const float DesiredYaw = DesiredMoveDir2D.Rotation().Yaw;
						const float DeltaYaw = DesiredYaw - GetActorRotation().Yaw;
						Cmd.RotationInput.Yaw = DeltaYaw / DeltaTimeSeconds;
					}
				}

				Cmd.MovementInput = FVector(FMath::Clamp<float>(MoveInput.Size2D(), -1.f, 1.f ), 0.0f, MoveInput.Z);
			}
			break;
		}
		case 2:
		{
			// Free camera on yaw and pitch, camera-relative movement.
			APlayerController* PC = Cast<APlayerController>(GetController());
			if (ensure(PC)) // Requires player controller for now
			{
				// Camera yaw rotation
				PC->AddYawInput(LookInput.X * ControllerLookRateYaw );
				PC->AddPitchInput(-LookInput.Y * ControllerLookRatePitch );

				// Rotational movement: orientate us towards our camera-relative desired velocity (unless we are upside down, then flip it)
				const float RotationMagMin = (1e-3f);
				const float MoveInputMag = MoveInput.Size();

				float RotationInputYaw = 0.f;
				float RotationInputPitch = 0.f;
				float RotationInputRoll = 0.f;

				if (MoveInputMag >= RotationMagMin)
				{								
					FVector DesiredMovementDir = PC->GetControlRotation().RotateVector(MoveInput);

					const float DesiredYaw = DesiredMovementDir.Rotation().Yaw;
					const float DeltaYaw = DesiredYaw - GetActorRotation().Yaw;

					const float DesiredPitch = DesiredMovementDir.Rotation().Pitch;
					const float DeltaPitch = DesiredPitch - GetActorRotation().Pitch;

					const float DesiredRoll = DesiredMovementDir.Rotation().Roll;
					const float DeltaRoll = DesiredRoll - GetActorRotation().Roll;

					// Kind of gross but because we want "instant" turning we must factor in delta time so that it gets factored out inside the simulation
					RotationInputYaw = DeltaYaw / DeltaTimeSeconds;
					RotationInputPitch = DeltaPitch / DeltaTimeSeconds;
					RotationInputRoll = DeltaRoll / DeltaTimeSeconds;
								
				}

				Cmd.RotationInput.Yaw = RotationInputYaw;
				Cmd.RotationInput.Pitch = RotationInputPitch;
				Cmd.RotationInput.Roll = RotationInputRoll;
						
				Cmd.MovementInput = FVector(FMath::Clamp<float>( MoveInputMag, -1.f, 1.f ), 0.0f, 0.0f); // Not the best way but simple
			}
			break;
		}
		case 3:
		{
			// Free camera on the yaw, camera-relative motion
			APlayerController* PC = Cast<APlayerController>(GetController());
			if (ensure(PC)) // Requires player controller for now
			{
				// Camera yaw rotation
				PC->AddYawInput(LookInput.X * ControllerLookRateYaw );

				// Rotational movement: orientate us towards our camera-relative desired velocity (unless we are upside down, then flip it)
				const float RotationMagMin = (1e-3f);
				const float MoveInputMag = MoveInput.Size2D();

				float RotationInputYaw = 0.f;
				if (MoveInputMag >= RotationMagMin)
				{
					FVector DesiredMovementDir = PC->GetControlRotation().RotateVector(MoveInput);

					const bool bIsUpsideDown = FVector::DotProduct(FVector(0.f, 0.f, 1.f), GetActorQuat().GetUpVector() ) < 0.f;
					if (bIsUpsideDown)
					{
						DesiredMovementDir *= -1.f;
					}							

					const float DesiredYaw = DesiredMovementDir.Rotation().Yaw;
					const float DeltaYaw = DesiredYaw - GetActorRotation().Yaw;

					// Kind of gross but because we want "instant" turning we must factor in delta time so that it gets factored out inside the simulation
					RotationInputYaw = DeltaYaw / DeltaTimeSeconds;
				}

				Cmd.RotationInput.Yaw = RotationInputYaw;
				Cmd.RotationInput.Pitch = LookInput.Y * LookRatePitch; // Just pitch like normal
				Cmd.RotationInput.Roll = 0;
						
				Cmd.MovementInput = FVector(FMath::Clamp<float>( MoveInputMag, -1.f, 1.f ), 0.0f, MoveInput.Z); // Not the best way but simple
			}
			break;
		}
	}				
}

// ------------------------------------------------------------------------

ANetworkPredictionExtrasCharacter_MockAbility::ANetworkPredictionExtrasCharacter_MockAbility(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UMockCharacterAbilityComponent>(Name_CharacterMotionComponent))
{

}

UMockCharacterAbilityComponent* ANetworkPredictionExtrasCharacter_MockAbility::GetMockCharacterAbilityComponent()
{
	return Cast<UMockCharacterAbilityComponent>(CharacterMotionComponent);
}

const UMockCharacterAbilityComponent* ANetworkPredictionExtrasCharacter_MockAbility::GetMockCharacterAbilityComponent() const
{
	return Cast<UMockCharacterAbilityComponent>(CharacterMotionComponent);
}

void ANetworkPredictionExtrasCharacter_MockAbility::ProduceInput(const int32 SimTimeMS, FMockCharacterAbilityInputCmd& Cmd)
{
	Super::ProduceInput(SimTimeMS, Cmd);

	Cmd.bSprintPressed = false;
	Cmd.bDashPressed = false;
	Cmd.bBlinkPressed = false;
	Cmd.bJumpPressed = false;
	Cmd.bPrimaryPressed = false;
	Cmd.bSecondaryPressed = false;

	switch(AbilityInputPreset)
	{
	case ENetworkPredictionExtrasMockCharacterAbilityInputPreset::None:
		Cmd.bSprintPressed = bSprintPressed;
		Cmd.bDashPressed = bDashPressed;
		Cmd.bBlinkPressed = bBlinkPressed;
		Cmd.bJumpPressed = bJumpPressed;
		Cmd.bPrimaryPressed = bPrimaryPressed;
		Cmd.bSecondaryPressed = bSecondaryPressed;
		break;
	case ENetworkPredictionExtrasMockCharacterAbilityInputPreset::Sprint:
		Cmd.bSprintPressed = true;
		break;
	case ENetworkPredictionExtrasMockCharacterAbilityInputPreset::Dash:
		Cmd.bDashPressed = true;
		break;
	case ENetworkPredictionExtrasMockCharacterAbilityInputPreset::Blink:
		Cmd.bBlinkPressed = true;
		break;
	case ENetworkPredictionExtrasMockCharacterAbilityInputPreset::Jump:
		Cmd.bJumpPressed = true;
		break;
	};
}

void ANetworkPredictionExtrasCharacter_MockAbility::BeginPlay()
{
	Super::BeginPlay();

	if (UWorld* World = GetWorld())
	{
		if (UMockCharacterAbilityComponent* CharacterAbilityComponent = GetMockCharacterAbilityComponent())
		{
			CharacterAbilityComponent->ProduceInputDelegate.BindUObject(this, &ANetworkPredictionExtrasCharacter_MockAbility::ProduceInput);
		}
	}
}

float ANetworkPredictionExtrasCharacter_MockAbility::GetStamina() const
{
	if (const UMockCharacterAbilityComponent* CharacterAbilityComponent = GetMockCharacterAbilityComponent())
	{
		return CharacterAbilityComponent->GetStamina();
	}
	return 0.f;
}

float ANetworkPredictionExtrasCharacter_MockAbility::GetMaxStamina() const
{
	if (const UMockCharacterAbilityComponent* CharacterAbilityComponent = GetMockCharacterAbilityComponent())
	{
		return CharacterAbilityComponent->GetMaxStamina();
	}
	return 0.f;
}

void ANetworkPredictionExtrasCharacter_MockAbility::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	Super::SetupPlayerInputComponent(PlayerInputComponent);

	if (APlayerController* PC = Cast<APlayerController>(GetController()))
	{
		if (PC->PlayerInput)
		{
			//Gamepad
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Sprint"), EKeys::Gamepad_LeftThumbstick));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Dash"), EKeys::Gamepad_FaceButton_Left));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Blink"), EKeys::Gamepad_FaceButton_Top));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Jump"), EKeys::Gamepad_FaceButton_Bottom));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Primary"), EKeys::Gamepad_RightShoulder));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Secondary"), EKeys::Gamepad_LeftShoulder));

			// Keyboard
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Sprint"), EKeys::LeftShift));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Dash"), EKeys::Q));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Blink"), EKeys::E));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Jump"), EKeys::SpaceBar));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Primary"), EKeys::LeftMouseButton));
			PC->PlayerInput->AddActionMapping(FInputActionKeyMapping(TEXT("Secondary"), EKeys::RightMouseButton));
		}
	}

	PlayerInputComponent->BindAction(TEXT("Sprint"), IE_Pressed, this, &ThisClass::Action_Sprint_Pressed);
	PlayerInputComponent->BindAction(TEXT("Sprint"), IE_Released, this, &ThisClass::Action_Sprint_Released);

	PlayerInputComponent->BindAction(TEXT("Dash"), IE_Pressed, this, &ThisClass::Action_Dash_Pressed);
	PlayerInputComponent->BindAction(TEXT("Dash"), IE_Released, this, &ThisClass::Action_Dash_Released);

	PlayerInputComponent->BindAction(TEXT("Blink"), IE_Pressed, this, &ThisClass::Action_Blink_Pressed);
	PlayerInputComponent->BindAction(TEXT("Blink"), IE_Released, this, &ThisClass::Action_Blink_Released);

	PlayerInputComponent->BindAction(TEXT("Jump"), IE_Pressed, this, &ThisClass::Action_Jump_Pressed);
	PlayerInputComponent->BindAction(TEXT("Jump"), IE_Released, this, &ThisClass::Action_Jump_Released);

	PlayerInputComponent->BindAction(TEXT("Primary"), IE_Pressed, this, &ThisClass::Action_Primary_Pressed);
	PlayerInputComponent->BindAction(TEXT("Primary"), IE_Released, this, &ThisClass::Action_Primary_Released);

	PlayerInputComponent->BindAction(TEXT("Secondary"), IE_Pressed, this, &ThisClass::Action_Secondary_Pressed);
	PlayerInputComponent->BindAction(TEXT("Secondary"), IE_Released, this, &ThisClass::Action_Secondary_Released);
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Sprint_Pressed()
{
	bSprintPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Sprint_Released()
{
	bSprintPressed = false;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Dash_Pressed()
{
	bDashPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Dash_Released()
{
	bDashPressed = false;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Blink_Pressed()
{
	bBlinkPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Blink_Released()
{
	bBlinkPressed = false;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Jump_Pressed()
{
	bJumpPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Jump_Released()
{
	bJumpPressed = false;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Primary_Pressed()
{
	bPrimaryPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Primary_Released()
{
	bPrimaryPressed = false;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Secondary_Pressed()
{
	bSecondaryPressed = true;
}

void ANetworkPredictionExtrasCharacter_MockAbility::Action_Secondary_Released()
{
	bSecondaryPressed = false;
}
