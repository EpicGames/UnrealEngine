// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionExtrasModule.h"
#include "Misc/CoreDelegates.h"
#include "MockRootMotionSourceObject.h"

#if WITH_EDITOR
#include "Editor.h"
#endif


#define LOCTEXT_NAMESPACE "FNetworkPredictionModule"

class FNetworkPredictionExtrasModule : public INetworkPredictionExtrasModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FDelegateHandle PieHandle;
	FDelegateHandle PostEngineInitHandle;
};

void FNetworkPredictionExtrasModule::StartupModule()
{
	PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
	{
		UMockRootMotionSourceClassMap::InitSingleton();
	});

#if WITH_EDITOR
	PieHandle = FEditorDelegates::PreBeginPIE.AddLambda([this](const bool bBegan)
	{
		UMockRootMotionSourceClassMap::BuildClassMap();
	});

#endif
}


void FNetworkPredictionExtrasModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	PostEngineInitHandle.Reset();

#if WITH_EDITOR
	FEditorDelegates::PreBeginPIE.Remove(PieHandle);
	PieHandle.Reset();
#endif
}

IMPLEMENT_MODULE( FNetworkPredictionExtrasModule, NetworkPredictionExtras )
#undef LOCTEXT_NAMESPACE

