// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "NetworkPredictionComponent.h"
#include "BaseMovementComponent.generated.h"

struct FHitResult;

// -------------------------------------------------------------------------------------------------------------------------------
//	Base component for movement. This essentially has the generic glue for selecting an UpdatedComponent and moving it along the world
//	It is abstract in that you still need to define which simulation the component runs (via ::InstantiateNetworkedSimulation)
// -------------------------------------------------------------------------------------------------------------------------------
UCLASS(Abstract)
class NETWORKPREDICTIONEXTRAS_API UBaseMovementComponent : public UNetworkPredictionComponent
{
	GENERATED_BODY()

public:

	UBaseMovementComponent();

	virtual void InitializeComponent() override;
	virtual void OnRegister() override;
	virtual void RegisterComponentTickFunctions(bool bRegister) override;

	// Callbacks 
	UFUNCTION()
	virtual void OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult) { }

	// Used by NetworkPrediction driver for physics interpolation case
	UPrimitiveComponent* GetPhysicsPrimitiveComponent() const { return UpdatedPrimitive; }

protected:

	// Basic "Update Component/Ticking"
	virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent);
	virtual void UpdateTickRegistration();

	UFUNCTION()
	virtual void PhysicsVolumeChanged(class APhysicsVolume* NewVolume);	

	UPROPERTY()
	TObjectPtr<USceneComponent> UpdatedComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> UpdatedPrimitive = nullptr;

private:

	/** Transient flag indicating whether we are executing OnRegister(). */
	bool bInOnRegister = false;
	
	/** Transient flag indicating whether we are executing InitializeComponent(). */
	bool bInInitializeComponent = false;
};