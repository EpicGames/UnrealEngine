// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"

class AActor;
class UPrimitiveComponent;

struct FCollisionQueryParams;
struct FCollisionResponseParams;
struct FCollisionShape;
struct FHitResult;

enum class ETeleportType : uint8;
enum EMoveComponentFlags;

// This provides a base for movement related simulations: moving an "UpdatedComponent" around the world.
// There is no actual Update function provided here, it will be implemented by subclasses.
class NETWORKPREDICTIONEXTRAS_API FBaseMovementSimulation
{
public:
	bool SafeMoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport) const;
	bool MoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit, ETeleportType Teleport) const;
	FTransform GetUpdateComponentTransform() const;

	FVector GetPenetrationAdjustment(const FHitResult& Hit) const;

	bool OverlapTest(const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor) const;
	void InitCollisionParams(FCollisionQueryParams &OutParams, FCollisionResponseParams& OutResponseParam) const;
	bool ResolvePenetration(const FVector& ProposedAdjustment, const FHitResult& Hit, const FQuat& NewRotationQuat) const;

	/**  Flags that control the behavior of calls to MoveComponent() on our UpdatedComponent. */
	mutable EMoveComponentFlags MoveComponentFlags = MOVECOMP_NoFlags; // Mutable because we sometimes need to disable these flags ::ResolvePenetration. Better way may be possible

	void SetComponents(USceneComponent* InUpdatedComponent, UPrimitiveComponent* InPrimitiveComponent)
	{
		UpdatedComponent = InUpdatedComponent;
		UpdatedPrimitive = InPrimitiveComponent;
	}

protected:

	USceneComponent* UpdatedComponent = nullptr;
	UPrimitiveComponent* UpdatedPrimitive = nullptr;
};