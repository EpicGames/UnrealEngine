// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/HitResult.h"
#include "Misc/StringBuilder.h"
#include "BaseMovementSimulation.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionStateTypes.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionSimulation.h"

// -------------------------------------------------------------------------------------------------------------------------------
// 
// -------------------------------------------------------------------------------------------------------------------------------

// State the client generates
struct NETWORKPREDICTIONEXTRAS_API FCharacterMotionInputCmd
{
	// Input: "pure" input for this frame. At this level, frame time has not been accounted for. (E.g., "move straight" would be (1,0,0) regardless of frame time)
	FRotator RotationInput;
	FVector MovementInput;

	FCharacterMotionInputCmd()
		: RotationInput(ForceInitToZero)
		, MovementInput(ForceInitToZero)
	{ }

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << RotationInput;
		P.Ar << MovementInput;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MovementInput: X=%.2f Y=%.2f Z=%.2f\n", MovementInput.X, MovementInput.Y, MovementInput.Z);
		Out.Appendf("RotationInput: P=%.2f Y=%.2f R=%.2f\n", RotationInput.Pitch, RotationInput.Yaw, RotationInput.Roll);
	}
};

/** Movement mode */
enum class ECharacterMovementMode : uint8
{
	None,
	Walking,
	Falling
};


/** Data about the floor for walking movement, used by CharacterMovementComponent. */
struct NETWORKPREDICTIONEXTRAS_API FFloorTestResult
{
	/**
	* True if there was a blocking hit in the floor test that was NOT in initial penetration.
	* The HitResult can give more info about other circumstances.
	*/
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
	uint32 bBlockingHit : 1;

	/** True if the hit found a valid walkable floor. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
	uint32 bWalkableFloor : 1;

	/** The distance to the floor, computed from the swept capsule trace. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
	float FloorDist;

	/** Hit result of the test that found a floor. Includes more specific data about the point of impact and surface normal at that point. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
	FHitResult HitResult;

public:

	FFloorTestResult()
		: bBlockingHit(false)
		, bWalkableFloor(false)
		, FloorDist(0.f)
		, HitResult(1.f)
	{
	}

	/** Returns true if the floor result hit a walkable surface. */
	bool IsWalkableFloor() const
	{
		return bBlockingHit && bWalkableFloor;
	}

	void Clear()
	{
		bBlockingHit = false;
		bWalkableFloor = false;
		FloorDist = 0.f;
		HitResult.Reset(1.f, false);
	}

	/** Gets the distance to floor, either LineDist or FloorDist. */
	float GetDistanceToFloor() const
	{
		return FloorDist;
	}

	void SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor);
};



// State we are evolving frame to frame and keeping in sync
struct NETWORKPREDICTIONEXTRAS_API FCharacterMotionSyncState
{
	ECharacterMovementMode MovementMode;
	FVector Location;
	FVector Velocity;
	FRotator Rotation;

	FCharacterMotionSyncState()
		: MovementMode(ECharacterMovementMode::Walking)
		, Location(ForceInitToZero)
		, Velocity(ForceInitToZero)
		, Rotation(ForceInitToZero)
	{ }

	bool ShouldReconcile(const FCharacterMotionSyncState& AuthorityState) const;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << MovementMode;
		P.Ar << Location;
		P.Ar << Velocity;
		P.Ar << Rotation;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MovementMode: %u\n", uint32{EnumToUnderlyingType(MovementMode)});
		Out.Appendf("Loc: X=%.2f Y=%.2f Z=%.2f\n", Location.X, Location.Y, Location.Z);
		Out.Appendf("Vel: X=%.2f Y=%.2f Z=%.2f\n", Velocity.X, Velocity.Y, Velocity.Z);
		Out.Appendf("Rot: P=%.2f Y=%.2f R=%.2f\n", Rotation.Pitch, Rotation.Yaw, Rotation.Roll);
	}

	void Interpolate(const FCharacterMotionSyncState* From, const FCharacterMotionSyncState* To, float PCT)
	{
		MovementMode = To->MovementMode;

		static constexpr float TeleportThreshold = 1000.f * 1000.f;
		if (FVector::DistSquared(From->Location, To->Location) > TeleportThreshold)
		{
			*this = *To;
		}
		else
		{
			Location = FMath::Lerp(From->Location, To->Location, PCT);
			Velocity = FMath::Lerp(From->Velocity, To->Velocity, PCT);
			Rotation = FMath::Lerp(From->Rotation, To->Rotation, PCT);
		}
	}
};

// Auxiliary state that is input into the simulation.
struct NETWORKPREDICTIONEXTRAS_API FCharacterMotionAuxState
{
	float MaxSpeed = 1200.f;
	float TurningBoost = 8.f;
	float Deceleration = 8000.f;
	float Acceleration = 4000.f;
	float WalkableFloorZ = 0.71f;
	float FloorSweepDistance = 40.0f;

	bool ShouldReconcile(const FCharacterMotionAuxState& AuthorityState) const;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << MaxSpeed;
		P.Ar << TurningBoost;
		P.Ar << Deceleration;
		P.Ar << Acceleration;
		P.Ar << WalkableFloorZ;
		P.Ar << FloorSweepDistance;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MaxSpeed: %.2f\n", MaxSpeed);
		Out.Appendf("TurningBoost: %.2f\n", TurningBoost);
		Out.Appendf("Deceleration: %.2f\n", Deceleration);
		Out.Appendf("Acceleration: %.2f\n", Acceleration);
		Out.Appendf("WalkableFloorZ: %.3f\n", WalkableFloorZ);
		Out.Appendf("FloorSweepDistance: %.2f\n", FloorSweepDistance);
	}

	void Interpolate(const FCharacterMotionAuxState* From, const FCharacterMotionAuxState* To, float PCT)
	{
		// This is probably a good case where interpolating values is pointless and it could just snap
		// to the 'To' state.
		MaxSpeed = FMath::Lerp(From->MaxSpeed, To->MaxSpeed, PCT);
		TurningBoost = FMath::Lerp(From->TurningBoost, To->TurningBoost, PCT);
		Deceleration = FMath::Lerp(From->Deceleration, To->Deceleration, PCT);
		Acceleration = FMath::Lerp(From->Acceleration, To->Acceleration, PCT);
		WalkableFloorZ = FMath::Lerp(From->WalkableFloorZ, To->WalkableFloorZ, PCT);
		FloorSweepDistance = FMath::Lerp(From->FloorSweepDistance, To->FloorSweepDistance, PCT);
	}
};

using CharacterMotionStateTypes = TNetworkPredictionStateTypes<FCharacterMotionInputCmd, FCharacterMotionSyncState, FCharacterMotionAuxState>;

class NETWORKPREDICTIONEXTRAS_API FCharacterMotionSimulation : public FBaseMovementSimulation
{
public:

	/** Main update function */
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output);

	// Callbacks
	void OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult);

	// general tolerance value for rotation checks
	static constexpr float ROTATOR_TOLERANCE = (1e-3f);

	/** Dev tool to force simple mispredict */
	static bool ForceMispredict;

protected:

	// Called at start of simulation to avoid using old cached values.
	virtual void InvalidateCache();

	virtual bool IsExceedingMaxSpeed(const FVector& Velocity, float InMaxSpeed) const;

	virtual FVector ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const;
	virtual void TwoWallAdjust(FVector& OutDelta, const FHitResult& Hit, const FVector& OldHitNormal) const;
	virtual float SlideAlongSurface(const FVector& Delta, float Time, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact);

	// TODO: it seems necessary to pass the Input and Output to every function since it encapulates all the possible state we may need to do our calculations, but it makes the function signatures pretty large and annoying to override and when adding new ones.

	virtual void PerformMovement(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput);

	virtual void Movement_Walking(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput);
	virtual void Movement_Falling(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput);

	virtual void SetMovementMode(ECharacterMovementMode MovementMode, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output);

	virtual FRotator ComputeLocalRotation(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const;
	virtual FVector ComputeLocalInput(const FNetSimTimeStep& TimeStep, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const;

	// Computes a velocity based on InputVelocity and current state
	virtual FVector ComputeVelocity(float DeltaSeconds, const FVector& InitialVelocity, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& LocalSpaceMovementInput) const;
	virtual FVector ComputeGravity(float DeltaSeconds, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Delta) const;
	
	virtual void FindFloor(const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Location, FFloorTestResult& OutFloorResult) const;
	virtual void ComputeFloorDist(const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Location, FFloorTestResult& OutFloorResult, float SweepDistance) const;
	virtual bool FloorSweepTest(FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParam) const;

	virtual bool IsWalkable(const FHitResult& Hit, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output) const;
	virtual bool IsValidLandingSpot(const FVector& Location, const FHitResult& Hit, const TNetSimInput<CharacterMotionStateTypes>& Input, const TNetSimOutput<CharacterMotionStateTypes>& Output, const FVector& Delta) const;


protected:

	FFloorTestResult CachedFloor;
};

