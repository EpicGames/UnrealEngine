// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMovementComponent.h"
#include "Templates/PimplPtr.h"
#include "FlyingMovementComponent.generated.h"

struct FFlyingMovementInputCmd;
struct FFlyingMovementSyncState;
struct FFlyingMovementAuxState;

class FFlyingMovementSimulation;

// -------------------------------------------------------------------------------------------------------------------------------
// ActorComponent for running FlyingMovement 
// -------------------------------------------------------------------------------------------------------------------------------

UCLASS(BlueprintType, meta=(BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UFlyingMovementComponent : public UBaseMovementComponent
{
	GENERATED_BODY()

public:

	UFlyingMovementComponent();

	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	// Forward input producing event to someone else (probably the owning actor)
	DECLARE_DELEGATE_TwoParams(FProduceFlyingInput, const int32 /*SimTime*/, FFlyingMovementInputCmd& /*Cmd*/)
	FProduceFlyingInput ProduceInputDelegate;

	// BeginOverlap has to be bound to a ufunction, so we have no choice but to bind here and forward into simulation code. Not ideal.
	virtual void OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult);

	// --------------------------------------------------------------------------------
	// NP Driver
	// --------------------------------------------------------------------------------

	// Get latest local input prior to simulation step
	void ProduceInput(const int32 DeltaTimeMS, FFlyingMovementInputCmd* Cmd);

	// Restore a previous frame prior to resimulating
	void RestoreFrame(const FFlyingMovementSyncState* SyncState, const FFlyingMovementAuxState* AuxState);

	// Take output for simulation
	void FinalizeFrame(const FFlyingMovementSyncState* SyncState, const FFlyingMovementAuxState* AuxState);

	// Seed initial values based on component's state
	void InitializeSimulationState(FFlyingMovementSyncState* Sync, FFlyingMovementAuxState* Aux);

	float GetMaxMoveSpeed() const;
	void SetMaxMoveSpeed(float NewMaxMoveSpeed);
	void AddMaxMoveSpeed(float AdditiveMaxMoveSpeed);

protected:

	// Network Prediction
	virtual void InitializeNetworkPredictionProxy() override;
	TPimplPtr<FFlyingMovementSimulation> OwnedMovementSimulation; // If we instantiate the sim in InitializeNetworkPredictionProxy, its stored here
	FFlyingMovementSimulation* ActiveMovementSimulation = nullptr; // The sim driving us, set in InitFlyingMovementSimulation. Could be child class that implements InitializeNetworkPredictionProxy.

	void InitFlyingMovementSimulation(FFlyingMovementSimulation* Simulation);

	static float GetDefaultMaxSpeed();
};