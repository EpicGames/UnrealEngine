// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StringBuilder.h"
#include "BaseMovementSimulation.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionStateTypes.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionSimulation.h"

// -------------------------------------------------------------------------------------------------------------------------------
// FlyingMovement: simple flying movement that was based on UE's FloatingPawnMovement
// -------------------------------------------------------------------------------------------------------------------------------

// State the client generates
struct FFlyingMovementInputCmd
{
	// Input: "pure" input for this frame. At this level, frame time has not been accounted for. (E.g., "move straight" would be (1,0,0) regardless of frame time)
	FRotator RotationInput;
	FVector MovementInput;

	FFlyingMovementInputCmd()
		: RotationInput(ForceInitToZero)
		, MovementInput(ForceInitToZero)
	{ }

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << RotationInput;
		P.Ar << MovementInput;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MovementInput: X=%.2f Y=%.2f Z=%.2f\n", MovementInput.X, MovementInput.Y, MovementInput.Z);
		Out.Appendf("RotationInput: P=%.2f Y=%.2f R=%.2f\n", RotationInput.Pitch, RotationInput.Yaw, RotationInput.Roll);
	}
};

// State we are evolving frame to frame and keeping in sync
struct FFlyingMovementSyncState
{
	FVector Location;
	FVector Velocity;
	FRotator Rotation;

	FFlyingMovementSyncState()
		: Location(ForceInitToZero)
		, Velocity(ForceInitToZero)
		, Rotation(ForceInitToZero)
	{ }

	bool ShouldReconcile(const FFlyingMovementSyncState& AuthorityState) const;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Location;
		P.Ar << Velocity;
		P.Ar << Rotation;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Loc: X=%.2f Y=%.2f Z=%.2f\n", Location.X, Location.Y, Location.Z);
		Out.Appendf("Vel: X=%.2f Y=%.2f Z=%.2f\n", Velocity.X, Velocity.Y, Velocity.Z);
		Out.Appendf("Rot: P=%.2f Y=%.2f R=%.2f\n", Rotation.Pitch, Rotation.Yaw, Rotation.Roll);
	}

	void Interpolate(const FFlyingMovementSyncState* From, const FFlyingMovementSyncState* To, float PCT)
	{
		static constexpr float TeleportThreshold = 1000.f * 1000.f;
		if (FVector::DistSquared(From->Location, To->Location) > TeleportThreshold)
		{
			*this = *To;
		}
		else
		{
			Location = FMath::Lerp(From->Location, To->Location, PCT);
			Velocity = FMath::Lerp(From->Velocity, To->Velocity, PCT);
			Rotation = FMath::Lerp(From->Rotation, To->Rotation, PCT);
		}
	}
};

// Auxiliary state that is input into the simulation.
struct FFlyingMovementAuxState
{	
	float MaxSpeed = 1200.f;
	float TurningBoost = 8.f;
	float Deceleration = 8000.f;
	float Acceleration = 4000.f;

	bool ShouldReconcile(const FFlyingMovementAuxState& AuthorityState) const;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << MaxSpeed;
		P.Ar << TurningBoost;
		P.Ar << Deceleration;
		P.Ar << Acceleration;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MaxSpeed: %.2f\n", MaxSpeed);
		Out.Appendf("TurningBoost: %.2f\n", TurningBoost);
		Out.Appendf("Deceleration: %.2f\n", Deceleration);
		Out.Appendf("Acceleration: %.2f\n", Acceleration);
	}

	void Interpolate(const FFlyingMovementAuxState* From, const FFlyingMovementAuxState* To, float PCT)
	{
		// This is probably a good case where interpolating values is pointless and it could just snap
		// to the 'To' state.
		MaxSpeed = FMath::Lerp(From->MaxSpeed, To->MaxSpeed, PCT);
		TurningBoost = FMath::Lerp(From->TurningBoost, To->TurningBoost, PCT);
		Deceleration = FMath::Lerp(From->Deceleration, To->Deceleration, PCT);
		Acceleration = FMath::Lerp(From->Acceleration, To->Acceleration, PCT);
	}
};

using FlyingMovementStateTypes = TNetworkPredictionStateTypes<FFlyingMovementInputCmd, FFlyingMovementSyncState, FFlyingMovementAuxState>;

class FFlyingMovementSimulation : public FBaseMovementSimulation
{
public:

	/** Main update function */
	NETWORKPREDICTIONEXTRAS_API void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<FlyingMovementStateTypes>& Input, const TNetSimOutput<FlyingMovementStateTypes>& Output);

	// Callbacks
	void OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult);

	// general tolerance value for rotation checks
	static constexpr float ROTATOR_TOLERANCE = (1e-3f);

	/** Dev tool to force simple mispredict */
	static bool ForceMispredict;

protected:

	float SlideAlongSurface(const FVector& Delta, float Time, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact);
};