// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "WorldCollision.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Misc/OutputDevice.h"
#include "Misc/CoreDelegates.h"
#include "Templates/PimplPtr.h"

#include "NetworkPredictionComponent.h"
#include "NetworkPredictionCueTraits.h"
#include "NetworkPredictionLog.h"

#include "FlyingMovementComponent.h"
#include "FlyingMovementSimulation.h"
#include "NetworkPredictionCues.h"

#include "MockAbilitySimulation.generated.h"

// -------------------------------------------------------------------------------------------------------------------------------
// Mock Ability Simulation
//	This is meant to illustrate how a higher level simulation can build off an existing one. While something like GameplayAbilities
//	is more generic and data driven, this illustrates how it will need to solve core issues.
//
//	This implements:
//	1. Stamina "attribute": basic attribute with max + regen value that is consumed by abilities.
//	2. Sprint: Increased max speed while sprint button is held. Drains stamina each frame.
//	3. Dash: Immediate acceleration to a high speed for X seconds.
//	4. Blink: Teleport to location X units ahead
//
// -------------------------------------------------------------------------------------------------------------------------------

// -------------------------------------------------------
// MockAbility Data structures
// -------------------------------------------------------

struct FMockAbilityInputCmd : public FFlyingMovementInputCmd
{
	bool bSprintPressed = false;
	bool bDashPressed = false;
	bool bBlinkPressed = false;

	bool bPrimaryPressed = false;
	bool bSecondaryPressed = false;

	void NetSerialize(const FNetSerializeParams& P)
	{
		FFlyingMovementInputCmd::NetSerialize(P);
		// Fixme: compress to single byte
		P.Ar << bSprintPressed;
		P.Ar << bDashPressed;
		P.Ar << bBlinkPressed;
		P.Ar << bPrimaryPressed;
		P.Ar << bSecondaryPressed;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FFlyingMovementInputCmd::ToString(Out);
		Out.Appendf("bSprintPressed: %d\n", bSprintPressed);
		Out.Appendf("bDashPressed: %d\n", bDashPressed);
		Out.Appendf("bBlinkPressed: %d\n", bBlinkPressed);
		Out.Appendf("bPrimaryPressed: %d\n", bPrimaryPressed);
		Out.Appendf("bSecondaryPressed: %d\n", bSecondaryPressed);
	}
};

struct FMockAbilitySyncState : public FFlyingMovementSyncState
{
	float Stamina = 0.f;
	
	void NetSerialize(const FNetSerializeParams& P)
	{
		FFlyingMovementSyncState::NetSerialize(P);
		P.Ar << Stamina;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FFlyingMovementSyncState::ToString(Out);
		Out.Appendf("Stamina: %.2f\n", Stamina);
	}
	
	void Interpolate(const FMockAbilitySyncState* From, const FMockAbilitySyncState* To, float PCT)
	{
		FFlyingMovementSyncState::Interpolate(From, To, PCT);

		Stamina = FMath::Lerp(From->Stamina, To->Stamina, PCT);
	}

	bool ShouldReconcile(const FMockAbilitySyncState& AuthorityState) const;
};

struct FMockAbilityAuxState : public FFlyingMovementAuxState
{
	float MaxStamina = 100.f;
	float StaminaRegenRate = 20.f;
	int16 DashTimeLeft = 0;
	int16 BlinkWarmupLeft = 0;
	int16 PrimaryCooldown = 0;
	bool bIsSprinting = false;

	void NetSerialize(const FNetSerializeParams& P)
	{
		FFlyingMovementAuxState::NetSerialize(P);
		P.Ar << MaxStamina;
		P.Ar << StaminaRegenRate;
		P.Ar << DashTimeLeft;
		P.Ar << BlinkWarmupLeft;
		P.Ar << bIsSprinting;
		P.Ar << PrimaryCooldown;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FFlyingMovementAuxState::ToString(Out);
		Out.Appendf("MaxStamina: %.2f\n", MaxStamina);
		Out.Appendf("StaminaRegenRate: %.2f\n", StaminaRegenRate);
		Out.Appendf("DashTimeLeft: %d\n", DashTimeLeft);
		Out.Appendf("BlinkWarmupLeft: %d\n", BlinkWarmupLeft);
		Out.Appendf("bIsSprinting: %d\n", bIsSprinting);
		Out.Appendf("PrimaryCooldown: %d\n", PrimaryCooldown);
	}

	void Interpolate(const FMockAbilityAuxState* From, const FMockAbilityAuxState* To, float PCT)
	{
		FFlyingMovementAuxState::Interpolate(From, To, PCT);

		MaxStamina = FMath::Lerp(From->MaxStamina, To->MaxStamina, PCT);
		StaminaRegenRate = FMath::Lerp(From->StaminaRegenRate, To->StaminaRegenRate, PCT);
		DashTimeLeft = FMath::Lerp(From->DashTimeLeft, To->DashTimeLeft, PCT);
		BlinkWarmupLeft = FMath::Lerp(From->BlinkWarmupLeft, To->BlinkWarmupLeft, PCT);
		PrimaryCooldown = FMath::Lerp(From->PrimaryCooldown, To->PrimaryCooldown, PCT);
		bIsSprinting = From->bIsSprinting;
	}

	bool ShouldReconcile(const FMockAbilityAuxState& AuthorityState) const;
};

// -------------------------------------------------------
// MockAbility NetSimCues - events emitted by the sim
// -------------------------------------------------------

// Cue for blink activation (the moment the ability starts)
struct FMockAbilityBlinkActivateCue
{
	FMockAbilityBlinkActivateCue() = default;
	FMockAbilityBlinkActivateCue(const FVector& InDestination, uint8 InRandomType)
		: Destination(InDestination), RandomType(InRandomType) { }

	NETSIMCUE_BODY();

	FVector_NetQuantize10 Destination;
	uint8 RandomType; // Random value used to color code the effect. This is the test/prove out mispredictions

	using Traits = NetSimCueTraits::Strong;
	
	void NetSerialize(FArchive& Ar)
	{
		bool b = false;
		Destination.NetSerialize(Ar, nullptr, b);
		Ar << RandomType;
	}
	
	bool NetIdentical(const FMockAbilityBlinkActivateCue& Other) const
	{
		const float ErrorTolerance = 1.f;
		return Destination.Equals(Other.Destination, ErrorTolerance) && RandomType == Other.RandomType;
	}
};

static_assert( TNetSimCueTraits<FMockAbilityBlinkActivateCue>::ReplicationTarget == NetSimCueTraits::Strong::ReplicationTarget, "Traits error" );

struct FMockAbilityPhysicsGunFireCue
{
	FMockAbilityPhysicsGunFireCue() = default;
	FMockAbilityPhysicsGunFireCue(const FVector& InStart, const FVector& InEnd, bool bInHasCooldown, TArray<FVector_NetQuantize100>&& InHitLocations)
		: Start(InStart), End(InEnd), bHasCooldown(bInHasCooldown), HitLocations(InHitLocations) { }

	NETSIMCUE_BODY();

	FVector_NetQuantize100 Start;
	FVector_NetQuantize100 End;
	bool bHasCooldown;

	TArray<FVector_NetQuantize100> HitLocations;

	using Traits = NetSimCueTraits::ReplicatedXOrPredicted;

	void NetSerialize(FArchive& Ar)
	{
		static constexpr int32 MaxItems = 3;

		bool b = false;
		Start.NetSerialize(Ar, nullptr, b);
		End.NetSerialize(Ar, nullptr, b);
		Ar << bHasCooldown;

		if (Ar.IsSaving())
		{
			uint8 Num = FMath::Min(MaxItems, HitLocations.Num());
			Ar << Num;

			for (int32 i=0; i < Num; ++i)
			{
				HitLocations[i].NetSerialize(Ar, nullptr, b);
			}
		}
		else
		{
			uint8 Num = 0;
			Ar << Num;
			Num = FMath::Min<uint8>(Num, MaxItems);
			HitLocations.SetNum(Num);
			for (int32 i=0; i < Num; ++i)
			{
				HitLocations[i].NetSerialize(Ar, nullptr, b);
			}
		}
	}

	bool NetIdentical(const FMockAbilityPhysicsGunFireCue& Other) const
	{
		const float ErrorTolerance = 1.f;
		return Start.Equals(Other.Start, ErrorTolerance) && End.Equals(Other.End, ErrorTolerance);
	}
};


// ----------------------------------------------------------------------------------------------

#define LOG_BLINK_CUE 0 // During development, its useful to sanity check that we aren't doing more construction or moves than we expect

// Cue for blink (the moment the teleport happens)
struct FMockAbilityBlinkCue
{
	FMockAbilityBlinkCue()
	{ 
		UE_CLOGF(LOG_BLINK_CUE, LogNetworkPrediction, Warning, "  Default Constructor 0x%llX", this);
	}

	FMockAbilityBlinkCue(const FVector& Start, const FVector& Stop) : StartLocation(Start), StopLocation(Stop)
	{ 
		UE_CLOGF(LOG_BLINK_CUE, LogNetworkPrediction, Warning, "  Custom Constructor 0x%llX", this);
	}

	~FMockAbilityBlinkCue()
	{ 
		UE_CLOGF(LOG_BLINK_CUE, LogNetworkPrediction, Warning, "  Destructor 0x%llX", this);
	}
	
	FMockAbilityBlinkCue(FMockAbilityBlinkCue&& Other)
		: StartLocation(MoveTemp(Other.StartLocation)), StopLocation(Other.StopLocation)
	{
		UE_CLOGF(LOG_BLINK_CUE, LogNetworkPrediction, Warning, "  Move Constructor 0x%llX (Other: 0x%llX)", this, &Other);
	}
	
	FMockAbilityBlinkCue& operator=(FMockAbilityBlinkCue&& Other)
	{
		UE_CLOGF(LOG_BLINK_CUE, LogNetworkPrediction, Warning, "  Move assignment 0x%llX (Other: 0x%llX)", this, &Other);
		StartLocation = MoveTemp(Other.StartLocation);
		StopLocation = MoveTemp(Other.StopLocation);
		return *this;
	}

	FMockAbilityBlinkCue(const FMockAbilityBlinkCue& Other) = delete;
	FMockAbilityBlinkCue& operator=(const FMockAbilityBlinkCue& Other) = delete;

	NETSIMCUE_BODY();

	FVector_NetQuantize10 StartLocation;
	FVector_NetQuantize10 StopLocation;

	using Traits = NetSimCueTraits::Strong;
	
	void NetSerialize(FArchive& Ar)
	{
		bool b = false;
		StartLocation.NetSerialize(Ar, nullptr, b);
		StopLocation.NetSerialize(Ar, nullptr, b);
	}
	
	
	bool NetIdentical(const FMockAbilityBlinkCue& Other) const
	{
		const float ErrorTolerance = 1.f;
		return StartLocation.Equals(Other.StartLocation, ErrorTolerance) && StopLocation.Equals(Other.StopLocation, ErrorTolerance);
	}
};

// -----------------------------------------------------------------------------------------------------
// Subtypes of the BlinkCue - this is not an expected setup! This is done for testing/debugging so we can 
// see the differences between the cue type traits in a controlled setup. See FMockAbilitySimulation::SimulationTick
// -----------------------------------------------------------------------------------------------------
#define DECLARE_BLINKCUE_SUBTYPE(TYPE, TRAITS) \
 struct TYPE : public FMockAbilityBlinkCue { \
 template <typename... ArgsType> TYPE(ArgsType&&... Args) : FMockAbilityBlinkCue(Forward<ArgsType>(Args)...) { } \
 using Traits = TRAITS; \
 void NetSerialize(FArchive& Ar) { FMockAbilityBlinkCue::NetSerialize(Ar); } \
 bool NetIdentical(const TYPE& Other) const { return FMockAbilityBlinkCue::NetIdentical(Other); } \
 NETSIMCUE_BODY(); };
 

DECLARE_BLINKCUE_SUBTYPE(FMockAbilityBlinkCue_Weak, NetSimCueTraits::Weak);
DECLARE_BLINKCUE_SUBTYPE(FMockAbilityBlinkCue_ReplicatedNonPredicted, NetSimCueTraits::ReplicatedNonPredicted);
DECLARE_BLINKCUE_SUBTYPE(FMockAbilityBlinkCue_ReplicatedXOrPredicted, NetSimCueTraits::ReplicatedXOrPredicted);
DECLARE_BLINKCUE_SUBTYPE(FMockAbilityBlinkCue_Strong, NetSimCueTraits::Strong);

// The set of Cues the MockAbility simulation will invoke
struct FMockAbilityCueSet
{
	template<typename TDispatchTable>
	static void RegisterNetSimCueTypes(TDispatchTable& DispatchTable)
	{
		DispatchTable.template RegisterType<FMockAbilityBlinkActivateCue>();
		DispatchTable.template RegisterType<FMockAbilityBlinkCue>();
		DispatchTable.template RegisterType<FMockAbilityPhysicsGunFireCue>();

		// (Again, not a normal setup, just for debugging/testing purposes)
		DispatchTable.template RegisterType<FMockAbilityBlinkCue_Weak>();
		DispatchTable.template RegisterType<FMockAbilityBlinkCue_ReplicatedNonPredicted>();
		DispatchTable.template RegisterType<FMockAbilityBlinkCue_ReplicatedXOrPredicted>();
		DispatchTable.template RegisterType<FMockAbilityBlinkCue_Strong>();
	}
};

// -------------------------------------------------------
// MockAbilitySimulation definition
// -------------------------------------------------------

using TMockAbilityBufferTypes = TNetworkPredictionStateTypes<FMockAbilityInputCmd, FMockAbilitySyncState, FMockAbilityAuxState>;

class FMockAbilitySimulation : public FFlyingMovementSimulation
{
public:

	/** Main update function */
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockAbilityBufferTypes>& Input, const TNetSimOutput<TMockAbilityBufferTypes>& Output);
};

// -------------------------------------------------------------------------------------------------------------------------------
// ActorComponent for running Mock Ability Simulation 
// -------------------------------------------------------------------------------------------------------------------------------

UCLASS(BlueprintType, meta=(BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UMockFlyingAbilityComponent : public UFlyingMovementComponent
{
	GENERATED_BODY()

public:

	UMockFlyingAbilityComponent();

	DECLARE_DELEGATE_TwoParams(FProduceMockAbilityInput, const int32 /*SimTime*/, FMockAbilityInputCmd& /*Cmd*/)
	FProduceMockAbilityInput ProduceInputDelegate;

	virtual void ProduceInput(const int32 SimTimeMS, FMockAbilityInputCmd* Cmd);
	virtual void FinalizeFrame(const FMockAbilitySyncState* SyncState, const FMockAbilityAuxState* AuxState);
	virtual void InitializeSimulationState(FMockAbilitySyncState* SyncState, FMockAbilityAuxState* AuxState);

	// NetSimCues
	void HandleCue(const FMockAbilityBlinkActivateCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters);
	void HandleCue(const FMockAbilityBlinkCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters);
	void HandleCue(const FMockAbilityPhysicsGunFireCue& FireCue, const FNetSimCueSystemParamemters& SystemParameters);

	// -------------------------------------------------------------------------------------
	//	Ability State and Notifications
	//		-This allows user code/blueprints to respond to state changes.
	//		-These values always reflect the latest simulation state
	//		-StateChange events are just that: when the state changes. They are not emitted from the sim themselves.
	//			-This means they "work" for interpolated simulations and are resilient to packet loss and crazy network conditions
	//			-That said, its "latest" only. There is NO guarantee that you will receive every state transition
	//
	// -------------------------------------------------------------------------------------

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMockAbilityNotifyStateChange, bool, bNewStateValue);

	// Notifies when Sprint state changes
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityNotifyStateChange OnSprintStateChange;

	// Notifies when Dash state changes
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityNotifyStateChange OnDashStateChange;

	// Notifies when Blink Changes
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityNotifyStateChange OnBlinkStateChange;

	// Are we currently in the sprinting state
	UFUNCTION(BlueprintCallable, Category="Mock AbilitySystem")
	bool IsSprinting() const { return bIsSprinting; }

	// Are we currently in the dashing state
	UFUNCTION(BlueprintCallable, Category="Mock AbilitySystem")
	bool IsDashing() const { return bIsDashing; }

	// Are we currently in the blinking (startup) state
	UFUNCTION(BlueprintCallable, Category="Mock AbilitySystem")
	bool IsBlinking() const { return bIsBlinking; }
	
	// Blueprint assignable events for blinking. THis allows the user/blueprint to implement rollback-able events
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMockAbilityBlinkCueEvent, FVector, DestinationLocation, int32, RandomValue, float, ElapsedTimeSeconds);
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityBlinkCueEvent OnBlinkActivateEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMockAbilityBlinkCueRollback);
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityBlinkCueRollback OnBlinkActivateEventRollback;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FMockAbilityPhysicsGunFireEvent, FVector, Start, FVector, End, bool, bHasCooldown, const TArray<FVector_NetQuantize100>&, HitLocations, float, ElapsedTimeSeconds);
	UPROPERTY(BlueprintAssignable, Category="Mock AbilitySystem")
	FMockAbilityPhysicsGunFireEvent OnPhysicsGunFirEvent;

	UFUNCTION(BlueprintCallable, Category="Mock AbilitySystem")
	float GetBlinkWarmupTimeSeconds() const;

	UFUNCTION(BlueprintCallable, Category="Mock Ability")
	float GetStamina() const;

	UFUNCTION(BlueprintCallable, Category="Mock Ability")
	float GetMaxStamina() const;

protected:

	// Network Prediction
	virtual void InitializeNetworkPredictionProxy() override;

	TPimplPtr<FMockAbilitySimulation> OwnedAbilitySimulation;
	FMockAbilitySimulation* ActiveAbilitySimulation = nullptr;
	
	void InitMockAbilitySimulation(FMockAbilitySimulation* Simulation);

private:

	// Local cached values for detecting state changes from the sim in ::FinalizeFrame
	// Its tempting to think ::FinalizeFrame could pass in the previous frames values but this could
	// not be reliable if buffer sizes are small and network conditions etc - you may not always know
	// what was the "last finalized frame" or even have it in the buffers anymore.
	bool bIsSprinting = false;
	bool bIsDashing = false;
	bool bIsBlinking = false;
};