// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CharacterMotionSimulation.h"
#include "CharacterMotionComponent.h"
#include "MockCharacterAbilitySimulation.generated.h"





// -------------------------------------------------------
// MockCharacterAbilitySimulation definition
// -------------------------------------------------------


struct FMockCharacterAbilityInputCmd : public FCharacterMotionInputCmd
{
	bool bSprintPressed = false;
	bool bDashPressed = false;
	bool bBlinkPressed = false;
	bool bJumpPressed = false;

	bool bPrimaryPressed = false;
	bool bSecondaryPressed = false;

	void NetSerialize(const FNetSerializeParams& P)
	{
		FCharacterMotionInputCmd::NetSerialize(P);
		// Fixme: compress to single byte
		P.Ar << bSprintPressed;
		P.Ar << bDashPressed;
		P.Ar << bBlinkPressed;
		P.Ar << bJumpPressed;
		P.Ar << bPrimaryPressed;
		P.Ar << bSecondaryPressed;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FCharacterMotionInputCmd::ToString(Out);
		Out.Appendf("bSprintPressed: %d\n", bSprintPressed);
		Out.Appendf("bDashPressed: %d\n", bDashPressed);
		Out.Appendf("bBlinkPressed: %d\n", bBlinkPressed);
		Out.Appendf("bJumpPressed: %d\n", bJumpPressed);
		Out.Appendf("bPrimaryPressed: %d\n", bPrimaryPressed);
		Out.Appendf("bSecondaryPressed: %d\n", bSecondaryPressed);
	}
};


struct FMockCharacterAbilitySyncState : public FCharacterMotionSyncState
{
	float Stamina = 0.f;

	void NetSerialize(const FNetSerializeParams& P)
	{
		FCharacterMotionSyncState::NetSerialize(P);
		P.Ar << Stamina;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FCharacterMotionSyncState::ToString(Out);
		Out.Appendf("Stamina: %.2f\n", Stamina);
	}

	void Interpolate(const FMockCharacterAbilitySyncState* From, const FMockCharacterAbilitySyncState* To, float PCT)
	{
		FCharacterMotionSyncState::Interpolate(From, To, PCT);

		Stamina = FMath::Lerp(From->Stamina, To->Stamina, PCT);
	}

	bool ShouldReconcile(const FMockCharacterAbilitySyncState& AuthorityState) const;
};

struct FMockCharacterAbilityAuxState : public FCharacterMotionAuxState
{
	float MaxStamina = 100.f;
	float StaminaRegenRate = 20.f;
	int16 DashTimeLeft = 0;
	int16 BlinkWarmupLeft = 0;
	int16 PrimaryCooldown = 0;
	bool bIsSprinting = false;
	bool bIsJumping = false;

	void NetSerialize(const FNetSerializeParams& P)
	{
		FCharacterMotionAuxState::NetSerialize(P);
		P.Ar << MaxStamina;
		P.Ar << StaminaRegenRate;
		P.Ar << DashTimeLeft;
		P.Ar << BlinkWarmupLeft;
		P.Ar << bIsSprinting;
		P.Ar << bIsJumping;
		P.Ar << PrimaryCooldown;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		FCharacterMotionAuxState::ToString(Out);
		Out.Appendf("MaxStamina: %.2f\n", MaxStamina);
		Out.Appendf("StaminaRegenRate: %.2f\n", StaminaRegenRate);
		Out.Appendf("DashTimeLeft: %d\n", DashTimeLeft);
		Out.Appendf("BlinkWarmupLeft: %d\n", BlinkWarmupLeft);
		Out.Appendf("bIsSprinting: %d\n", bIsSprinting);
		Out.Appendf("bIsJumping: %d\n", bIsJumping);
		Out.Appendf("PrimaryCooldown: %d\n", PrimaryCooldown);
	}

	void Interpolate(const FMockCharacterAbilityAuxState* From, const FMockCharacterAbilityAuxState* To, float PCT)
	{
		FCharacterMotionAuxState::Interpolate(From, To, PCT);

		MaxStamina = FMath::Lerp(From->MaxStamina, To->MaxStamina, PCT);
		StaminaRegenRate = FMath::Lerp(From->StaminaRegenRate, To->StaminaRegenRate, PCT);
		DashTimeLeft = FMath::Lerp(From->DashTimeLeft, To->DashTimeLeft, PCT);
		BlinkWarmupLeft = FMath::Lerp(From->BlinkWarmupLeft, To->BlinkWarmupLeft, PCT);
		PrimaryCooldown = FMath::Lerp(From->PrimaryCooldown, To->PrimaryCooldown, PCT);
		bIsSprinting = From->bIsSprinting;
		bIsJumping = From->bIsJumping;
	}

	bool ShouldReconcile(const FMockCharacterAbilityAuxState& AuthorityState) const;
};



// -------------------------------------------------------
// MockCharacterAbilitySimulation definition
// -------------------------------------------------------

using TMockCharacterAbilityBufferTypes = TNetworkPredictionStateTypes<FMockCharacterAbilityInputCmd, FMockCharacterAbilitySyncState, FMockCharacterAbilityAuxState>;

class FMockCharacterAbilitySimulation : public FCharacterMotionSimulation
{
	typedef FCharacterMotionSimulation Super;

public:

	/** Main update function */
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockCharacterAbilityBufferTypes>& Input, const TNetSimOutput<TMockCharacterAbilityBufferTypes>& Output);
};




// -------------------------------------------------------------------------------------------------------------------------------
// ActorComponent for running Mock Ability Simulation for CharacterMotion example
// -------------------------------------------------------------------------------------------------------------------------------

UCLASS(BlueprintType, meta = (BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UMockCharacterAbilityComponent : public UCharacterMotionComponent
{
	GENERATED_BODY()

public:

	UMockCharacterAbilityComponent();

	DECLARE_DELEGATE_TwoParams(FProduceMockCharacterAbilityInput, const int32 /*SimTime*/, FMockCharacterAbilityInputCmd& /*Cmd*/)
	FProduceMockCharacterAbilityInput ProduceInputDelegate;

	virtual void ProduceInput(const int32 SimTimeMS, FMockCharacterAbilityInputCmd* Cmd);
	virtual void FinalizeFrame(const FMockCharacterAbilitySyncState* SyncState, const FMockCharacterAbilityAuxState* AuxState);
	virtual void InitializeSimulationState(FMockCharacterAbilitySyncState* SyncState, FMockCharacterAbilityAuxState* AuxState);

	/*
	// NetSimCues
	void HandleCue(const FMockCharacterAbilityBlinkActivateCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters);
	void HandleCue(const FMockCharacterAbilityBlinkCue& BlinkCue, const FNetSimCueSystemParamemters& SystemParameters);
	void HandleCue(const FMockCharacterAbilityPhysicsGunFireCue& FireCue, const FNetSimCueSystemParamemters& SystemParameters);
	*/

	// -------------------------------------------------------------------------------------
	//	Ability State and Notifications
	//		-This allows user code/blueprints to respond to state changes.
	//		-These values always reflect the latest simulation state
	//		-StateChange events are just that: when the state changes. They are not emitted from the sim themselves.
	//			-This means they "work" for interpolated simulations and are resilient to packet loss and crazy network conditions
	//			-That said, its "latest" only. There is NO guarantee that you will receive every state transition
	//
	// -------------------------------------------------------------------------------------

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMockCharacterAbilityNotifyStateChange, bool, bNewStateValue);

	// Notifies when Sprint state changes
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityNotifyStateChange OnSprintStateChange;

	// Notifies when Dash state changes
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityNotifyStateChange OnDashStateChange;

	// Notifies when Blink Changes
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityNotifyStateChange OnBlinkStateChange;

	// Notifies when Jump Changes
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityNotifyStateChange OnJumpStateChange;

	// Are we currently in the sprinting state
	UFUNCTION(BlueprintCallable, Category = "Mock AbilitySystem") 
	bool IsSprinting() const { return bIsSprinting; }

	// Are we currently in the dashing state
	UFUNCTION(BlueprintCallable, Category = "Mock AbilitySystem")
	bool IsDashing() const { return bIsDashing; }

	// Are we currently in the blinking (startup) state
	UFUNCTION(BlueprintCallable, Category = "Mock AbilitySystem")
	bool IsBlinking() const { return bIsBlinking; }

	// Are we currently in the jumping state
	UFUNCTION(BlueprintCallable, Category = "Mock AbilitySystem")
	bool IsJumping() const { return bIsJumping; }

	// Blueprint assignable events for blinking. This allows the user/blueprint to implement rollback-able events
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMockCharacterAbilityBlinkCueEvent, FVector, DestinationLocation, int32, RandomValue, float, ElapsedTimeSeconds);
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityBlinkCueEvent OnBlinkActivateEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMockCharacterAbilityBlinkCueRollback);
	UPROPERTY(BlueprintAssignable, Category = "Mock AbilitySystem")
	FMockCharacterAbilityBlinkCueRollback OnBlinkActivateEventRollback;

	UFUNCTION(BlueprintCallable, Category = "Mock AbilitySystem")
	float GetBlinkWarmupTimeSeconds() const;

	UFUNCTION(BlueprintCallable, Category = "Mock Ability")
	float GetStamina() const;

	UFUNCTION(BlueprintCallable, Category = "Mock Ability")
	float GetMaxStamina() const;

protected:

	// Network Prediction
	virtual void InitializeNetworkPredictionProxy() override;

	TPimplPtr<FMockCharacterAbilitySimulation> OwnedAbilitySimulation;
	FMockCharacterAbilitySimulation* ActiveAbilitySimulation = nullptr;

	void InitMockCharacterAbilitySimulation(FMockCharacterAbilitySimulation* Simulation);

private:

	// Local cached values for detecting state changes from the sim in ::FinalizeFrame
	// Its tempting to think ::FinalizeFrame could pass in the previous frames values but this could
	// not be reliable if buffer sizes are small and network conditions etc - you may not always know
	// what was the "last finalized frame" or even have it in the buffers anymore.
	bool bIsSprinting = false;
	bool bIsDashing = false;
	bool bIsBlinking = false;
	bool bIsJumping = false;
};


