// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "WorldCollision.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Misc/OutputDevice.h"
#include "Misc/CoreDelegates.h"

#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionCues.h"
#include "NetworkPredictionStateTypes.h"
#include "NetworkPredictionSimulation.h"
#include "NetworkPredictionModelDef.h"
#include "NetworkPredictionComponent.h"
#include "Misc/StringBuilder.h"
#include "Stats/Stats.h"
#include "NetworkPredictionTrace.h"

#include "MockNetworkSimulation.generated.h"

DECLARE_STATS_GROUP(TEXT("NetworkPrediction"), STATGROUP_NetworkPrediction, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("NetworkPrediction_MockSimTick"), STAT_NetworkPrediction_MockSimTick, STATGROUP_NetworkPrediction);

// -------------------------------------------------------------------------------------------------------------------------------
//	Mock Network Simulation
//
//	This provides a minimal "mock" example of using the Network Prediction system. The simulation being run by these classes
//	is a simple accumulator that takes random numbers (FMockInputCmd::InputValue) as input. There is no movement related functionality
//	in this example. This is just meant to show the bare minimum hook ups into the system to make it easier to understand.
//
//	Highlights:
//		FMockNetworkModel::SimulationTick		The "core update" function of the simulation.
//		UMockNetworkSimulationComponent			The UE Actor Component that anchors the system to an actor.
//
//	Usage:
//		You can just add a UMockNetworkSimulationComponent to any ROLE_AutonomousProxy actor yourself (Default Subobject, through blueprints, manually, etc)
//		The console command "mns.Spawn" can be used to dynamically spawn the component on every pawn. Must be run on the server or in single process PIE.
//
//	Once spawned, there are some useful console commands that can be used. These bind to number keys by default (toggle-able via mns.BindAutomatically).
//		[Five] "mns.DoLocalInput 1"			can be used to submit random local input into the accumulator. This is how you advance the simulation.
//		[Six]  "mns.RequestMispredict 1"	can be used to force a mis predict (random value added to accumulator server side). Useful for tracing through the correction/resimulate code path.
//		[Nine] "nms.Debug.LocallyControlledPawn"	toggle debug hud for the locally controlled player.
//		[Zero] "nms.Debug.ToggleContinous"			Toggles continuous vs snapshotted display of the debug hud
//
//	Notes:
//		Everything is crammed into MockNetworkSimulation.h/.cpp. It may make sense to break the simulation and component code into
//		separate files for more complex simulations.
//
// -------------------------------------------------------------------------------------------------------------------------------


// -------------------------------------------------------------------------------------------------------------------------------
//	Simulation State
// -------------------------------------------------------------------------------------------------------------------------------

// State the client generates
struct FMockInputCmd
{
	float InputValue=0;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << InputValue;
	}
	
	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("InputValue: %.4f\n", InputValue);
	}
};

// State we are evolving frame to frame and keeping in sync
struct FMockSyncState
{
	float Total=0;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Total;
	}

	// Compare this state with AuthorityState. return true if a reconcile (correction) should happen
	bool ShouldReconcile(const FMockSyncState& AuthorityState) const
	{
		UE_NP_TRACE_RECONCILE(FMath::Abs<float>(Total - AuthorityState.Total) > SMALL_NUMBER, "Total:");
		return false;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Total: %.4f\n", Total);
	}

	void Interpolate(const FMockSyncState* From, const FMockSyncState* To, float PCT)
	{
		Total = FMath::Lerp(From->Total, To->Total, PCT);
	}
};

// Auxiliary state that is input into the simulation. Doesn't change during the simulation tick.
// (It can change and even be predicted but doing so will trigger more bookeeping, etc. Changes will happen "next tick")
struct FMockAuxState
{
	float Multiplier=1;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Multiplier;
	}

	bool ShouldReconcile(const FMockAuxState& Authority) const
	{
		UE_NP_TRACE_RECONCILE(Multiplier != Authority.Multiplier, "Multiplier:");
		return false;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Multiplier: %.4f\n", Multiplier);
	}

	void Interpolate(const FMockAuxState* From, const FMockAuxState* To, float PCT)
	{
		Multiplier = FMath::Lerp(From->Multiplier, To->Multiplier, PCT);
	}
};

// -------------------------------------------------------------------------------------------------------------------------------
//	Simulation Cues
// -------------------------------------------------------------------------------------------------------------------------------

// A minimal Cue, emitted every 10 "totals" with a random integer as a payload
struct FMockCue
{
	FMockCue() = default;
	FMockCue(int32 InRand)
		: RandomData(InRand) { }

	NETSIMCUE_BODY();
	int32 RandomData = 0;
};

// Set of all cues the Mock simulation emits. Not strictly necessary and not that helpful when there is only one cue (but using since this is the example real simulations will want to use)
struct FMockCueSet
{
	template<typename TDispatchTable>
	static void RegisterNetSimCueTypes(TDispatchTable& DispatchTable)
	{
		DispatchTable.template RegisterType<FMockCue>();
	}
};

// -------------------------------------------------------------------------------------------------------------------------------
//	The Simulation
// -------------------------------------------------------------------------------------------------------------------------------

using TMockNetworkSimulationBufferTypes = TNetworkPredictionStateTypes<FMockInputCmd, FMockSyncState, FMockAuxState>;

struct FNetSimTimeStep;

class FMockNetworkSimulation
{
public:

	/** Main update function */
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TMockNetworkSimulationBufferTypes>& Input, const TNetSimOutput<TMockNetworkSimulationBufferTypes>& Output);
};

// -------------------------------------------------------------------------------------------------------------------------------
// ActorComponent for running a MockNetworkSimulation (implements Driver for the mock simulation)
// -------------------------------------------------------------------------------------------------------------------------------

UCLASS(BlueprintType, meta=(BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UMockNetworkSimulationComponent : public UNetworkPredictionComponent
{
	GENERATED_BODY()

public:

	UMockNetworkSimulationComponent();

	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	virtual void InitializeNetworkPredictionProxy() override;

	// Seed initial values based on component's state
	void InitializeSimulationState(FMockSyncState* Sync, FMockAuxState* Aux);

	// Set latest input
	void ProduceInput(const int32 DeltaTimeMS, FMockInputCmd* Cmd);

	// Take output of Sim and push to component
	void FinalizeFrame(const FMockSyncState* Sync, const FMockAuxState* Aux);
	
public:
	
	void HandleCue(const FMockCue& MockCue, const FNetSimCueSystemParamemters& SystemParameters);

	// Mock representation of "syncing' to the sync state in the network sim.
	float MockValue = 1000.f;

	// Our own simulation object. This could just as easily be shared among all UMockNetworkSimulationComponent
	TUniquePtr<FMockNetworkSimulation> MockNetworkSimulation;
};
