// Copyright Epic Games, Inc. All Rights Reserved

#pragma once
#include "BaseMovementComponent.h"
#include "MockRootMotionSimulation.h"
#include "MockRootMotionSourceStore.h"

#include "MockRootMotionComponent.generated.h"

class UAnimInstance;
class UAnimMontage;
class UCurveVector;
class UMockRootMotionSource;

USTRUCT()
struct FRootMotionSourceCache
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMockRootMotionSource> Instance = nullptr;

	int32 ClassID = 0;
};

// This component acts as the Driver for the FMockRootMotionSimulation
// It is essentially a standin for the movement component, and would be replaced by "new movement system" component.
// If we support "root motion without movement component" then this could either be that component, or possibly
// built into or inherit from a USkeletalMeshComponent.
//
// The main thing this provides is:
//		-Interface for initiating root motions through the NP system (via client Input and via server "OOB" writes)
//		-FinalizeFrame: take the output of the NP simulation and push it to the movement/animation components
//		-Place holder implementation of IRootMotionSourceStore(the temp thing that maps our RootMotionSourceIDs -> actual sources)

UCLASS(BlueprintType, meta=(BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UMockRootMotionComponent : public UBaseMovementComponent, public IRootMotionSourceStore
{
public:

	GENERATED_BODY()

	virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent) override;

	void InitializeSimulationState(FMockRootMotionSyncState* SyncState, FMockRootMotionAuxState* AuxState);
	void ProduceInput(const int32 SimTimeMS, FMockRootMotionInputCmd* Cmd);
	void RestoreFrame(const FMockRootMotionSyncState* SyncState, const FMockRootMotionAuxState* AuxState);
	void FinalizeFrame(const FMockRootMotionSyncState* SyncState, const FMockRootMotionAuxState* AuxState);	

	// ------------------------------------------------------------------------

	UFUNCTION(BlueprintCallable, Category=Input)
	UMockRootMotionSource* CreateRootMotionSource(TSubclassOf<UMockRootMotionSource> Source);

	UFUNCTION(BlueprintCallable, Category=Input)
	void Input_PlayRootMotionSource(UMockRootMotionSource* Source);

	UFUNCTION(BlueprintCallable, Category=Input)
	void Input_PlayRootMotionSourceByClass(TSubclassOf<UMockRootMotionSource> Source);

	// Callable by authority. Plays "out of band" animation: e.g, directly sets the RootMotionSourceID on the sync state, rather than the pending InputCmd.
	// This is analogous to outside code teleporting the actor (outside of the core simulation function)
	UFUNCTION(BlueprintCallable, Category=Animation)
	void PlayRootMotionSource(UMockRootMotionSource* Source);

	UFUNCTION(BlueprintCallable, Category=Animation)
	void PlayRootMotionSourceByClass(TSubclassOf<UMockRootMotionSource> Source);

	// Root Motion Object store
	UMockRootMotionSource* ResolveRootMotionSource(int32 ID, const TArrayView<const uint8>& Data) override;
	void StoreRootMotionSource(int32 ID, UMockRootMotionSource* Source) override;

protected:

	void FindAndCacheAnimInstance();
	
	// Next local InputCmd that will be submitted. This is a simple way of getting input to the system
	FMockRootMotionInputCmd PendingInputCmd;

	void InitializeNetworkPredictionProxy() override;
	TUniquePtr<FMockRootMotionSimulation> OwnedMockRootMotionSimulation;

	UAnimInstance* AnimInstance = nullptr;

private:

	UPROPERTY()
	FRootMotionSourceCache RootMotionSourceCache;
};