// Copyright Epic Games, Inc. All Rights Reserved

#pragma once
#include "Engine/EngineTypes.h"
#include "Misc/StringBuilder.h"
#include "NetworkPredictionStateTypes.h"
#include "NetworkPredictionSimulation.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionReplicationProxy.h"
#include "BaseMovementSimulation.h"
#include "MockRootMotionSourceStore.h"
#include "NetworkPredictionTrace.h"

class UAnimInstance;

// Thing that goes in the aux state
template<typename IdType=uint8, int32 InInlineSize=128, int32 InMaxSize=255>
struct TMockRootMotionSourceProxy
{
	enum { InlineSize = InInlineSize };
	enum { MaxSize = InMaxSize };
	enum { InvalidValue = TNumericLimits<IdType>::Max() };

	void Reset()
	{
		ID = InvalidValue;
		StartMS = 0;
		Params.Reset();
	}

	bool IsValid() const
	{
		return ID < InvalidValue;
	}

	// Writes the blob of parameters to the actual network stream
	void NetSerialize(const FNetSerializeParams& P)
	{
		npCheckf(Params.Num() <= MaxSize, TEXT("Parameter size too big %d"), Params.Num());

		P.Ar << ID;
		P.Ar << StartMS;

		if(P.Ar.IsSaving())
		{	
			uint8 Size = Params.Num();
			P.Ar << Size;
			P.Ar.Serialize(Params.GetData(), Params.Num());
		}
		else
		{
			uint8 Size = 0;
			P.Ar << Size;
			Params.SetNumUninitialized(Size, EAllowShrinking::No);
			P.Ar.Serialize(Params.GetData(), Size);
		}
	}

	FBitReader GetParamBitReader()
	{
		return FBitReader(Params.GetData(), Params.Num() << 3);
	}

	// Writes a new source with no params
	void WriteSource(IdType InID, int32 InStartMS)
	{
		npEnsureMsgf(InID < InvalidValue, TEXT("Invalid Value written: %d"), InID);

		ID = InID;
		StartMS = InStartMS;
		Params.Reset();
	}

	// Write a new source and encode the params into the generic param block
	void WriteParams(TFunctionRef<void(FBitWriter&)> WriteFunc)
	{
		static constexpr int64 MaxBits = (int64)MaxSize << 3;
		Params.Reset();

		// Unfortunate we cannot just create bit writer to our Params
		FBitWriter Writer(MaxBits, false);
		WriteFunc(Writer);

		Params = *Writer.GetBuffer();
	}

	IdType GetID() const { return ID; }
	int32 GetStartMS() const { return StartMS; }
	TArrayView<const uint8> GetParamsDataView() const { return TArrayView<const uint8>(Params); }

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("SourceID: %d\n", ID);
		Out.Appendf("StartMS: %d\n", StartMS);
		Out.Appendf("Params Size: %d\n", Params.Num());
	}

	bool operator==(const TMockRootMotionSourceProxy<IdType, InlineSize, MaxSize> &Other) const
	{
		return Params.Num() == Other.Params.Num() && FMemory::Memcmp(Params.GetData(), Other.Params.GetData(), Params.Num()) == 0;
	}

	bool operator!=(const TMockRootMotionSourceProxy<IdType, InlineSize, MaxSize> &Other) const { return !(*this == Other); }

private:
	IdType ID = InvalidValue;	// static mapping to the source (could be global or per sim/actor/anim instance)
	int32 StartMS = 0;			// Sim time this source started
	TArray<uint8, TInlineAllocator<InlineSize>> Params; // quantized params
};


// This is an initial prototype of root motion in the Network Prediction system. It is meant to flesh out some ideas before 
// settling on a final design for the future of root motion. In other words, we do not expect the code here in NetworkPredictionExtas
// to be used directly in shipping systems.

// High level idea:
//	-Get montage based root motion stood up
//	-Expand on the idea of "Root Motion Sources" meaning any kind of motion-driving logic that can be decoupled from the "character/pawn movement system"
//	-This would include simple curve based motions, programatically defined motion ("move towards actor"), or more complex, dynamic animation based motion.
//	-Eventually this folds back into the "new movement system" and/or possibly becomes something that can stand on its own without being driven by the former (TBD).

// Issues / Callouts:
//	-UObject* replication inside NP sync/aux state is not supported (Without disabling WithNetSharedSerialization)
//	
//

struct FMockRootMotionInputCmd
{
	// State that is generated by the client. Strictly speaking for RootMotion, an InputCmd doesn't
	// make sense - input is the concern of the higher level system that would decide to play 
	// RootMotions. For this mock example though, we'll make an InputCmd that can trigger an animation
	// to play from the client. That way, the client can initiate an animation predictively.
	//
	// The real world example would be more like "InputCmd says activate an abilty, the ability says
	// to play a montage".

	TMockRootMotionSourceProxy<> PlaySource;	// What we are want to play

	void NetSerialize(const FNetSerializeParams& P)
	{
		PlaySource.NetSerialize(P);
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		PlaySource.ToString(Out);
	}
};

struct FMockRootMotionSyncState
{
	// Transform state. In the final version we may want to decouple this from the animation state,
	// For example if a "movement simulation" was driving things, it may "own" the transform
	// and feed it into the root motion system. But this is meant to be a stand alone mock example.
	FVector Location;
	FRotator Rotation;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Location;
		P.Ar << Rotation;
	}
	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Loc: X=%.2f Y=%.2f Z=%.2f\n", Location.X, Location.Y, Location.Z);
		Out.Appendf("Rot: P=%.2f Y=%.2f R=%.2f\n", Rotation.Pitch, Rotation.Yaw, Rotation.Roll);
	}

	void Interpolate(const FMockRootMotionSyncState* From, const FMockRootMotionSyncState* To, float PCT)
	{
		static constexpr float TeleportThreshold = 1000.f * 1000.f;
		if (FVector::DistSquared(From->Location, To->Location) > TeleportThreshold)
		{
			*this = *To;
		}
		else
		{
			Location = FMath::Lerp(From->Location, To->Location, PCT);
			Rotation = FMath::Lerp(From->Rotation, To->Rotation, PCT);
		}
	}

	bool ShouldReconcile(const FMockRootMotionSyncState& AuthorityState) const
	{
		const float TransformErrorTolerance = 1.f;
		UE_NP_TRACE_RECONCILE(!Location.Equals(AuthorityState.Location, TransformErrorTolerance), "Location:");
		return false;
	}
};

// The aux state should hold state that does not frequently change. It is otherwise the same as sync state.
// (note that optimizations for sparse aux storage are not complete yet)
struct FMockRootMotionAuxState
{
	TMockRootMotionSourceProxy<> Source;
	int32 EndTimeMS = 0;	// Time the Source finished. Used in the sim's input handling

	void NetSerialize(const FNetSerializeParams& P)
	{
		Source.NetSerialize(P);
		P.Ar << EndTimeMS;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Source.ToString(Out);
		Out.Appendf("EndTimeMS: %d\n", EndTimeMS);
	}

	bool ShouldReconcile(const FMockRootMotionAuxState& AuthorityState) const
	{
		UE_NP_TRACE_RECONCILE(this->Source != AuthorityState.Source, "EndTimeMS:");
		return false;
	}

	void Interpolate(const FMockRootMotionAuxState* From, const FMockRootMotionAuxState* To, float PCT)
	{
		this->Source = To->Source;
	}
};

// This is the interface into "things that actually provide root motion"
class IMockRootMotionSourceMap
{
public:

	// Advance the root motion state by the given TimeStep
	virtual FTransform StepRootMotion(const FNetSimTimeStep& TimeStep, const FMockRootMotionSyncState* In, FMockRootMotionSyncState* Out, const FMockRootMotionAuxState* Aux) = 0;

	// Push the Sync state to the AnimInstance
	//	this is debatable - the simulation code doesn't need to call this, its really the concern of the driver (UMockRootMotionComponent)
	//	and not all potential root motion sources are going to want to set a pose. 
	virtual void FinalizePose(const FMockRootMotionSyncState* Sync, UAnimInstance* AnimInstance) = 0;
};

// This just defines the state types that the simulation uses
using MockRootMotionStateTypes = TNetworkPredictionStateTypes<FMockRootMotionInputCmd, FMockRootMotionSyncState, FMockRootMotionAuxState>;

// The actual NetworkPrediction simulation code that implements root motion movement
//	(root motion evaluation itself is done via IMockRootMotionSourceMap but the actual 'how to move thing given a delta' is done here)
class FMockRootMotionSimulation : public FBaseMovementSimulation
{
public:

	// The main tick function
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<MockRootMotionStateTypes>& Input, const TNetSimOutput<MockRootMotionStateTypes>& Output);

	IRootMotionSourceStore* SourceStore = nullptr;

	template<typename ProxyType>
	UMockRootMotionSource* ResolveRootMotionSource(const ProxyType& Proxy)
	{
		return SourceStore->ResolveRootMotionSource(Proxy.GetID(), Proxy.GetParamsDataView());
	}

	// The component the root motion is relative to. This was found to be needed since, in our examples, we author root motion anims where Y is forward
	// and we rotate the mesh components at the actor level so that X is forward. We need to know which component to rotate the root motion animation relative to.
	// If we continue with this, this means all non anim based root motions should expect to follow the same convention. Need clarity here from animation team.
	USceneComponent* RootMotionComponent = nullptr;
};