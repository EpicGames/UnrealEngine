// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "NetworkPredictionReplicationProxy.h" // FNetSerializeParams
#include "UObject/ObjectKey.h"
#include "Containers/SortedMap.h"
#include "Templates/SubclassOf.h"
#include "Serialization/BitWriter.h"
#include "Engine/DataAsset.h"
#include "GameFramework/Actor.h"
#include "MockRootMotionSourceObject.generated.h"

class UAnimInstance;
class UAnimMontage;
class UCurveVector;
class ANetworkPredictionReplicatedManager;

struct FMockRootMotionStepParameters
{
	const FVector& Location;
	const FRotator& Rotation;
	int32 ElapsedMS;	// Time that has been stepped since this source was played
	int32 DeltaMS;		// Delta time we are stepping with this call
};

struct FMockRootMotionReturnValue
{
	enum class ESourceState : uint8
	{
		Continue,
		Stop
	};

	enum class ETransformType : uint8
	{
		Absolute,
		AnimationRelative
	};

	ESourceState State = ESourceState::Continue;
	ETransformType TransformType = ETransformType::Absolute;
	FTransform DeltaTransform = FTransform::Identity;

	void Stop() { State = ESourceState::Stop; }
	void Continue() { State = ESourceState::Continue; }
};

UCLASS(abstract)
class UMockRootMotionSource : public UObject
{
	GENERATED_BODY()

public:

	UMockRootMotionSource() = default;
	virtual ~UMockRootMotionSource() { }

	virtual FMockRootMotionReturnValue Step(const FMockRootMotionStepParameters& Parameters) { return FMockRootMotionReturnValue(); }

	virtual void FinalizePose(int32 ElapsedMS, UAnimInstance* AnimInstance) const { }

	// Returns true if this instance is an acceptable runtime source for root motion.
	//	When called on a CDO, calls IsValidCDO()
	//	When called on a constructed instance, calls IsValidInstance();
	bool IsValidRootMotionSource() const
	{
		return (HasAnyFlags(RF_ClassDefaultObject) && IsValidCDO()) || IsValidInstance();
	}

	// Only called on CDOs - Is this class default object valid (or does it require subclassing to fill in required data)
	//	If true, this class gets a root motion source ID assigned and can be replicated
	//	If false, no ID and runtime error if this class is tried to be used
	virtual bool IsValidCDO() const { return false; }

	// Runtime check if this source is well formed/valid for use
	//	By default this will just check IsValidCDO, but subclasses that require instancing + dynamic data will want to check that.
	virtual bool IsValidInstance() const { return IsValidCDO(); }

	virtual void SerializePayloadParameters(FBitArchive& Ar) { }
};


UCLASS()
class UMockRootMotionSourceClassMap : public UObject
{
	GENERATED_BODY()

public:

	virtual void BeginDestroy() override;

	// Creates singleton instance
	static void InitSingleton();

	// Refreshes internal class structure: must be called when new classes are available/unloaded
	static void BuildClassMap();

	// Get Singleton instance
	static UMockRootMotionSourceClassMap* Get()
	{
		npCheckSlow(Singleton);
		return Singleton;
	}

	// Copy singleton's data into replicated manager 
	void CopyToReplicatedManager(ANetworkPredictionReplicatedManager* Manager);

private:
	
	void BuildClassMap_Internal();
	
	static UMockRootMotionSourceClassMap* Singleton;

	FDelegateHandle AuthoritySpawnCallbackHandle;

	UPROPERTY()
	TArray<TSoftClassPtr<UMockRootMotionSource>> SourceList;

	TSortedMap<UClass*, int32> LookupMap;
};

// ---------------------------------------------------------------
//
// ---------------------------------------------------------------

UCLASS(Blueprintable)
class UMockRootMotionSource_Montage : public UMockRootMotionSource
{
	GENERATED_BODY()

	// (BP) Subclasses must set montage asset for this to be valid
	virtual bool IsValidCDO() const { return Montage != nullptr; }

	void SerializePayloadParameters(FBitArchive& Ar) override
	{ 
		Ar << PlayRate;
	}

protected:

	UPROPERTY(EditAnywhere, Category=RootMotion)
	TObjectPtr<UAnimMontage> Montage;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	float PlayRate = 1.f;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	FVector TranslationScale = FVector(1.f);

	virtual FMockRootMotionReturnValue Step(const FMockRootMotionStepParameters& Parameters) override;

	void FinalizePose(int32 ElapsedMS, UAnimInstance* AnimInstance) const override;
};

UCLASS(Blueprintable)
class UMockRootMotionSource_Curve : public UMockRootMotionSource
{
	GENERATED_BODY()

	// (BP) Subclasses must set curve  asset for this to be valid
	virtual bool IsValidCDO() const { return Curve != nullptr; }

	void SerializePayloadParameters(FBitArchive& Ar) override
	{ 
		Ar << PlayRate;
	}

protected:

	UPROPERTY(EditAnywhere, Category=RootMotion)
	TObjectPtr<UCurveVector> Curve;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	float PlayRate = 1.f;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	FVector TranslationScale = FVector(1.f);

	virtual FMockRootMotionReturnValue Step(const FMockRootMotionStepParameters& Parameters);
};

UCLASS(Blueprintable)
class UMockRootMotionSource_MoveToLocation : public UMockRootMotionSource
{
	GENERATED_BODY()

	// This is always a valid source class: Destination is only set at runtime (we have no static data that must be set)
	virtual bool IsValidCDO() const { return true; }

	// Require Destination to be set for this instance to be valid
	virtual bool IsValidInstance() const { return bDestinationSet; }

	virtual FMockRootMotionReturnValue Step(const FMockRootMotionStepParameters& Parameters);

	void SerializePayloadParameters(FBitArchive& Ar) override
	{ 
		if (Ar.IsSaving())
		{
			npEnsure(bDestinationSet);
			Ar << Destination;
		}
		else
		{
			Ar << Destination;
			bDestinationSet = true;
		}
	}

	UFUNCTION(BlueprintCallable, Category="Root Motion")
	void SetDestination(const FVector& InDestination) { Destination = InDestination; bDestinationSet = true; }

protected:

	UPROPERTY(EditAnywhere, Category=RootMotion)
	FVector Destination;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	float Velocity = 100.f;

	UPROPERTY(EditAnywhere, Category=RootMotion)
	float SnapToTolerance = 1.f;

	bool bDestinationSet = false;
};