// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "Containers/ArrayView.h"

class UMockRootMotionSource;

// Interface for managing instantiated root motion source objects
class IRootMotionSourceStore
{
public:
	virtual UMockRootMotionSource* ResolveRootMotionSource(int32 ID, const TArrayView<const uint8>& Data) = 0;
	virtual void StoreRootMotionSource(int32 ID, UMockRootMotionSource* Source) = 0;
};