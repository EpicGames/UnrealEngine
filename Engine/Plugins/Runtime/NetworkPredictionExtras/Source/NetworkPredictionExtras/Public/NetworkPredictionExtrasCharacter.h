// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Pawn.h"
#include "CharacterMotionComponent.h"
#include "CharacterMotionSimulation.h"
#include "MockCharacterAbilitySimulation.h"
#include "NetworkPredictionExtrasCharacter.generated.h"

class UInputComponent;
class UCharacterMotionComponent;

// -------------------------------------------------------------------------------------------------------------------------------
//	ANetworkPredictionExtrasCharacter
//
//	This provides a minimal pawn class that uses UCharacterMotionCompnent. This isn't really intended to be used in shipping games,
//	rather just to serve as standalone example of using the system, contained completely in the NetworkPredictionExtras plugin.
//	Apart from the most basic glue/setup, this class provides an example of turning UE input event callbacks into the input commands
//	that are used by the Character movement simulation. This includes some basic camera/aiming code.
//
//	Highlights:
//		CharacterMotion::FMovementSystem::SimulationTick				The "core update" function of the Character movement simulation.
//		ANetworkPredictionExtrasCharacter::GenerateLocalInput		Function that generates local input commands that are fed into the movement system.
//
//	Usage:
//		You should be able to just use this pawn like you would any other pawn. You can specify it as your pawn class in your game mode, or manually override in world settings, etc.
//		Alternatively, you can just load the NetworkPredictionExtras/Content/TestMap.umap which will have everything setup.
//
//	Once spawned, there are some useful console commands:
//		NetworkPredictionExtras.Character.CameraSyle [0-3]			Changes camera mode style.
//		nms.Debug.LocallyControlledPawn 1							Enables debug hud. binds to '9' by default, see ANetworkPredictionExtrasCharacter()
//		nms.Debug.ToggleContinous 1									Toggles continuous updates of the debug hud. binds to '0' by default, see ANetworkPredictionExtrasCharacter()
//
// -------------------------------------------------------------------------------------------------------------------------------

UENUM()
enum class ENetworkPredictionExtrasCharacterInputPreset: uint8
{
	/** No input */
	None,
	/** Just moves forward */
	Forward
};

/** Sample pawn that uses UCharacterMotionComponent. The main thing this provides is actually producing user input for the component/simulation to consume. */
UCLASS()
class NETWORKPREDICTIONEXTRAS_API ANetworkPredictionExtrasCharacter : public APawn
{
	GENERATED_BODY()

public:

	ANetworkPredictionExtrasCharacter(const FObjectInitializer& ObjectInitializer);

	virtual void BeginPlay() override;
	virtual void SetupPlayerInputComponent(UInputComponent* PlayerInputComponent) override;
	virtual void Tick( float DeltaSeconds) override;
	virtual UNetConnection* GetNetConnection() const override; // For bFakeAutonomousProxy only


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Automation")
	ENetworkPredictionExtrasCharacterInputPreset InputPreset;

	/** Actor will behave like autonomous proxy even though not posessed by an APlayercontroller. To be used in conjuction with InputPreset. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Automation")
	bool bFakeAutonomousProxy = false;

	UFUNCTION(BlueprintCallable, Category="Debug")
	void PrintDebug();

	UFUNCTION(BlueprintCallable, Category="Gameplay")
	float GetMaxMoveSpeed() const;

	UFUNCTION(BlueprintCallable, Category="Gameplay")
	void SetMaxMoveSpeed(float NewMaxMoveSpeed);

	UFUNCTION(BlueprintCallable, Category="Gameplay")
	void AddMaxMoveSpeed(float AdditiveMaxMoveSpeed);

protected:

	void ProduceInput(const int32 DeltaMS, FCharacterMotionInputCmd& Cmd);

	UPROPERTY(Category=Movement, VisibleAnywhere)
	TObjectPtr<UCharacterMotionComponent> CharacterMotionComponent;

private:

	FVector CachedMoveInput;
	FVector2D CachedLookInput;

	void InputAxis_MoveForward(float Value);
	void InputAxis_MoveRight(float Value);
	void InputAxis_LookYaw(float Value);
	void InputAxis_LookPitch(float Value);
	void InputAxis_MoveUp(float Value);
	void InputAxis_MoveDown(float Value);

	void Action_LeftShoulder_Pressed() { }
	void Action_LeftShoulder_Released() { }
	void Action_RightShoulder_Pressed() { }
	void Action_RightShoulder_Released() { }
};

UENUM()
enum class ENetworkPredictionExtrasMockCharacterAbilityInputPreset: uint8
{
	/** No input */
	None,
	Sprint,
	Dash,
	Blink,
	Jump
};


// Example subclass of ANetworkPredictionExtrasCharacter that uses the MockAbility simulation
UCLASS()
class NETWORKPREDICTIONEXTRAS_API ANetworkPredictionExtrasCharacter_MockAbility : public ANetworkPredictionExtrasCharacter
{
	GENERATED_BODY()

public:

	ANetworkPredictionExtrasCharacter_MockAbility(const FObjectInitializer& ObjectInitializer);
	virtual void BeginPlay() override;
	virtual void SetupPlayerInputComponent(UInputComponent* PlayerInputComponent) override;

	
	UFUNCTION(BlueprintPure, Category="Ability")
	UMockCharacterAbilityComponent* GetMockCharacterAbilityComponent();

	const UMockCharacterAbilityComponent* GetMockCharacterAbilityComponent() const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Automation")
	ENetworkPredictionExtrasMockCharacterAbilityInputPreset AbilityInputPreset = ENetworkPredictionExtrasMockCharacterAbilityInputPreset::None;

	UFUNCTION(BlueprintCallable, Category="Ability")
	float GetStamina() const;

	UFUNCTION(BlueprintCallable, Category="Ability")
	float GetMaxStamina() const;
	
protected:
	using ANetworkPredictionExtrasCharacter::ProduceInput;
	void ProduceInput(const int32 SimTimeMS, FMockCharacterAbilityInputCmd& Cmd);

	void Action_Sprint_Pressed();
	void Action_Sprint_Released();
	void Action_Dash_Pressed();
	void Action_Dash_Released();
	void Action_Blink_Pressed();
	void Action_Blink_Released();
	void Action_Jump_Pressed();
	void Action_Jump_Released();

	void Action_Primary_Pressed();
	void Action_Primary_Released();

	void Action_Secondary_Pressed();
	void Action_Secondary_Released();

	bool bSprintPressed = false;
	bool bDashPressed = false;
	bool bBlinkPressed = false;
	bool bJumpPressed = false;
	bool bPrimaryPressed = false;
	bool bSecondaryPressed = false;
};

