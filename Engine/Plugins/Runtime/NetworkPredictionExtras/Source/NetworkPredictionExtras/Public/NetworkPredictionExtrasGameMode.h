// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/GameMode.h"
#include "Engine/Engine.h"
#include "NetworkPredictionExtrasGameMode.generated.h"

UCLASS(BlueprintType)
class ANetworkPredictionExtrasGameMode : public AGameMode
{
	GENERATED_BODY()

	ANetworkPredictionExtrasGameMode(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool AllowCheats(APlayerController* P) { return true; }
};