// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/GameState.h"
#include "Engine/Engine.h"
#include "NetworkPredictionExtrasGameState.generated.h"

UCLASS(BlueprintType)
class NETWORKPREDICTIONEXTRAS_API ANetworkPredictionExtrasGameState : public AGameState
{
	GENERATED_BODY()

	ANetworkPredictionExtrasGameState() = default;

	virtual void BeginPlay() override
	{
		Super::BeginPlay();

		GEngine->Exec(GetWorld(), TEXT("np2.DevMenu"));
	}
};