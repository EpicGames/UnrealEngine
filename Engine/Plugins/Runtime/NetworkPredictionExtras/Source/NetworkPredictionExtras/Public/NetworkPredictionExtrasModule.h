// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class INetworkPredictionExtrasModule : public IModuleInterface
{

public:
	
	static inline INetworkPredictionExtrasModule& Get()
	{
		return FModuleManager::LoadModuleChecked< INetworkPredictionExtrasModule >( "NetworkPredictionExtras" );
	}
	
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "NetworkPredictionExtras" );
	}
};

