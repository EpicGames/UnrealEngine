// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricMovement.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "Services/NetworkPredictionInstanceMap.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

DEFINE_LOG_CATEGORY_STATIC(LogParametricMovement, Log, All);

namespace ParametricMoverCVars
{
static int32 UseVLogger = 1;
static FAutoConsoleVariableRef CVarUseVLogger(TEXT("parametricmover.Debug.UseUnrealVLogger"),
	UseVLogger,	TEXT("Use Unreal Visual Logger\n"),	ECVF_Default);

static int32 UseDrawDebug = 1;
static FAutoConsoleVariableRef CVarUseDrawDebug(TEXT("parametricmover.Debug.UseDrawDebug"),
	UseVLogger,	TEXT("Use built in DrawDebug* functions for visual logging\n"), ECVF_Default);

static float DrawDebugDefaultLifeTime = 30.f;
static FAutoConsoleVariableRef CVarDrawDebugDefaultLifeTime(TEXT("parametricmover.Debug.UseDrawDebug.DefaultLifeTime"),
	DrawDebugDefaultLifeTime, TEXT("Use built in DrawDebug* functions for visual logging"), ECVF_Default);


static int32 SimulatedProxyBufferSize = 4;
static FAutoConsoleVariableRef CVarSimulatedProxyBufferSize(TEXT("parametricmover.SimulatedProxyBufferSize"),
	DrawDebugDefaultLifeTime, TEXT(""), ECVF_Default);

static int32 FixStep = 0;
static FAutoConsoleVariableRef CVarFixStepMS(TEXT("parametricmover.FixStep"),
	DrawDebugDefaultLifeTime, TEXT("If > 0, will use fix step"), ECVF_Default);

static float ErrorTolerance = 0.1f;
static FAutoConsoleVariableRef CVarErrorTolerance(TEXT("parametricmover.ErrorTolerance"),
	ErrorTolerance, TEXT("Error tolerance for reconcile"), ECVF_Default);

}

// -------------------------------------------------------------------------------------------------------------------------------
// FParametricMovementModelDef
// -------------------------------------------------------------------------------------------------------------------------------

/** NetworkedSimulation Model type */
class FParametricMovementModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using StateTypes = ParametricMovementBufferTypes;
	using Simulation = FParametricMovementSimulation;
	using Driver = UParametricMovementComponent;

	static const TCHAR* GetName() { return TEXT("Parametric"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers + 5; }
};

NP_MODEL_REGISTER(FParametricMovementModelDef);

bool FParametricAuxState::ShouldReconcile(const FParametricAuxState& AuthorityState) const
{
	const float MultiplierDelta = FMath::Abs<float>(AuthorityState.Multiplier - Multiplier);
	UE_NP_TRACE_RECONCILE(MultiplierDelta > ParametricMoverCVars::ErrorTolerance, "Multiplier:");
	return false;
}

bool FParametricSyncState::ShouldReconcile(const FParametricSyncState& AuthorityState) const
{
	const float PositionDelta = FMath::Abs<float>(AuthorityState.Position - Position);
	const float PlayRateDelta = FMath::Abs<float>(AuthorityState.PlayRate - PlayRate);


	UE_NP_TRACE_RECONCILE(PositionDelta > ParametricMoverCVars::ErrorTolerance, "PositionDelta:");
	UE_NP_TRACE_RECONCILE(PlayRateDelta > ParametricMoverCVars::ErrorTolerance, "PlayRateDelta:");

	return false;
}


// -------------------------------------------------------------------------------------------------------------------------------
// ParametricMovement
// -------------------------------------------------------------------------------------------------------------------------------

void FParametricMovementSimulation::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<ParametricMovementBufferTypes>& Input, const TNetSimOutput<ParametricMovementBufferTypes>& Output)
{
	if (!npEnsure(Motion))
	{
		return;
	}

	check(Motion != nullptr); // Must set motion mapping prior to running the simulation
	const float DeltaSeconds = (float)TimeStep.StepMS / 1000.f;

	// Advance parametric time. This won't always be linear: we could loop, rewind/bounce, etc
	const float InputPlayRate = Input.Cmd->PlayRate.Get(Input.Sync->PlayRate); // Returns InputCmds playrate if set, else returns previous state's playrate		
	Motion->AdvanceParametricTime(Input.Sync->Position, InputPlayRate, Output.Sync->Position, Output.Sync->PlayRate, DeltaSeconds);

	// We have our time that we should be at. We just need to move primitive component to that position.
	// Again, note that we expect this cannot fail. We move like this so that it can push things, but we don't expect failure.
	// (I think it would need to be a different simulation that supports this as a failure case. The tricky thing would be working out
	// the output Position in the case where a Move is blocked (E.g, you move 50% towards the desired location)

	FTransform StartingTransform;
	Motion->MapTimeToTransform(Input.Sync->Position, StartingTransform);

	FTransform NewTransform;
	Motion->MapTimeToTransform(Output.Sync->Position, NewTransform);

	FHitResult Hit(1.f);

	FVector Delta = NewTransform.GetLocation() - StartingTransform.GetLocation();
	MoveUpdatedComponent(Delta, NewTransform.GetRotation(), true, &Hit, ETeleportType::None);

	if (Hit.IsValidBlockingHit())
	{
		FTransform ActualTransform = GetUpdateComponentTransform();
		FVector ActualDelta = NewTransform.GetLocation() - ActualTransform.GetLocation();
		UE_LOGF(LogParametricMovement, Warning, "Blocking hit occurred when trying to move parametric mover. ActualDelta: %ls", *ActualDelta.ToString());
	}
}

// -------------------------------------------------------------------------------------------------------------------------------
// UParametricMovementComponent
// -------------------------------------------------------------------------------------------------------------------------------

UParametricMovementComponent::UParametricMovementComponent()
{

}

void UParametricMovementComponent::InitializeNetworkPredictionProxy()
{
	if (bDisableParametricMovementSimulation)
	{
		return;
	}

	OwnedParametricMovementSimulation = MakePimpl<FParametricMovementSimulation>();
	OwnedParametricMovementSimulation->Motion = &ParametricMotion;

	NetworkPredictionProxy.Init<FParametricMovementModelDef>(GetWorld(), GetReplicationProxies(), OwnedParametricMovementSimulation.Get(), this);
}

void UParametricMovementComponent::InitializeSimulationState(FParametricSyncState* SyncState, FParametricAuxState* AuxState)
{
}

void UParametricMovementComponent::BeginPlay()
{
	Super::BeginPlay();
	ParametricMotion.CachedStartingTransform = UpdatedComponent->GetComponentToWorld();
}

void UParametricMovementComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	if (bEnableForceNetUpdate)
	{
		GetOwner()->ForceNetUpdate();
	}
}

void UParametricMovementComponent::ProduceInput(const int32 DeltaTimeMS, FParametricInputCmd* Cmd)
{
	Cmd->PlayRate = PendingPlayRate;
	PendingPlayRate.Reset();
}

void UParametricMovementComponent::RestoreFrame(const FParametricSyncState* SyncState, const FParametricAuxState* AuxState)
{
	FTransform NewTransform;
	ParametricMotion.MapTimeToTransform(SyncState->Position, NewTransform);

	npEnsureSlow(NewTransform.ContainsNaN() == false);

	check(UpdatedComponent);
	UpdatedComponent->SetWorldTransform(NewTransform, false, nullptr, ETeleportType::TeleportPhysics);
}

void UParametricMovementComponent::FinalizeFrame(const FParametricSyncState* SyncState, const FParametricAuxState* AuxState)
{
	RestoreFrame(SyncState, AuxState);
}

void UParametricMovementComponent::EnableInterpolationMode(bool bValue)
{
	bEnableInterpolation = bValue;
}

// -------------------------------------------------------------------------------------------
// -------------------------------------------------------------------------------------------

void FSimpleParametricMotion::MapTimeToTransform(const float InPosition, FTransform& OutTransform) const
{
	const FVector Delta = ParametricDelta * InPosition;

	OutTransform = CachedStartingTransform;
	OutTransform.AddToTranslation(Delta);
}

void FSimpleParametricMotion::AdvanceParametricTime(const float InPosition, const float InPlayRate, float &OutPosition, float& OutPlayRate, const float DeltaTimeSeconds) const
{
	// Real simple oscillation for now
	OutPosition = InPosition + (InPlayRate * DeltaTimeSeconds);
	OutPlayRate = InPlayRate;

	const float DeltaMax = OutPosition - MaxTime;
	if (DeltaMax > SMALL_NUMBER)
	{
		OutPosition = MaxTime - DeltaMax;
		OutPlayRate *= -1.f;
	}
	else
	{
		const float DeltaMin = OutPosition - MinTime;
		if (DeltaMin < SMALL_NUMBER)
		{
			OutPosition = MinTime - DeltaMin;
			OutPlayRate *= -1.f;
		}
	}
}
