// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "BaseMovementComponent.h"
#include "NetworkPredictionStateTypes.h"
#include "Misc/StringBuilder.h"
#include "Templates/PimplPtr.h"

#include "NetworkPredictionReplicationProxy.h"
#include "BaseMovementSimulation.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionSimulation.h"

#include "ParametricMovement.generated.h"

// Extremely simple struct for defining parametric motion. This is editable in UParametricMovementComponent's defaults, and also used by the simulation code. 
USTRUCT(BlueprintType)
struct FSimpleParametricMotion
{
	GENERATED_BODY()

	// Actually turn the given position into a transform. Again, should be static and not conditional on changing state outside of the network sim
	void MapTimeToTransform(const float InPosition, FTransform& OutTransform) const;

	// Advance parametric time. This is meant to do simple things like looping/reversing etc.
	void AdvanceParametricTime(const float InPosition, const float InPlayRate, float &OutPosition, float& OutPlayRate, const float DeltaTimeSeconds) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParametricMovement)
	FVector ParametricDelta = FVector(0.f, 0.f, 500.f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParametricMovement)
	float MinTime = -1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParametricMovement)
	float MaxTime = 1.f;

	FTransform CachedStartingTransform;
};

// -------------------------------------------------------------------------------------------------------------------------------
//	Parametric Movement Simulation Types
// -------------------------------------------------------------------------------------------------------------------------------

// State the client generates
struct FParametricInputCmd
{
	// Input Playrate. This being set can be thought of "telling the simulation what its new playrate should be"
	TOptional<float> PlayRate;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << PlayRate;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		if (PlayRate.IsSet())
		{
			Out.Appendf("PlatRate: %.2f\n", PlayRate.GetValue());
		}
		else
		{
			Out.Appendf("PlayRate: Unset\n");
		}
	}
};

// State we are evolving frame to frame and keeping in sync
struct FParametricSyncState
{
	float Position=0.f;
	float PlayRate=1.f;

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Position;
		P.Ar << PlayRate;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Pos: %.2f\n", Position);
		Out.Appendf("Rate: %.2f\n", PlayRate);
	}

	void Interpolate(const FParametricSyncState* From, const FParametricSyncState* To, float PCT)
	{
		Position = FMath::Lerp(From->Position, To->Position, PCT);
		PlayRate = FMath::Lerp(From->PlayRate, To->PlayRate, PCT);
	}

	bool ShouldReconcile(const FParametricSyncState& AuthorityState) const;
};

// Auxiliary state that is input into the simulation. Doesn't change during the simulation tick.
// (It can change and even be predicted but doing so will trigger more bookeeping, etc. Changes will happen "next tick")
struct FParametricAuxState
{
	float Multiplier=1;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Multiplier;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Multiplier: %.2f\n", Multiplier);
	}

	void Interpolate(const FParametricAuxState* From, const FParametricAuxState* To, float PCT)
	{
		Multiplier = FMath::Lerp(From->Multiplier, To->Multiplier, PCT);
	}

	bool ShouldReconcile(const FParametricAuxState& AuthorityState) const;
};

/** BufferTypes for ParametricMovement */
using ParametricMovementBufferTypes = TNetworkPredictionStateTypes<FParametricInputCmd, FParametricSyncState, FParametricAuxState>;

/** The actual movement simulation */
class FParametricMovementSimulation : public FBaseMovementSimulation
{
public:

	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<ParametricMovementBufferTypes>& Input, const TNetSimOutput<ParametricMovementBufferTypes>& Output);

	// Pointer to our static mapping of time->position
	const FSimpleParametricMotion* Motion = nullptr;
};

// -------------------------------------------------------------------------------------------------------------------------------
//	ActorComponent for running basic Parametric movement. 
//	Parametric movement could be anything that takes a Time and returns an FTransform.
//	
//	Initially, we will support pushing (ie, we sweep as we update the mover's position).
//	But we will not allow a parametric mover from being blocked. 
//
// -------------------------------------------------------------------------------------------------------------------------------

UCLASS(BlueprintType, meta=(BlueprintSpawnableComponent))
class NETWORKPREDICTIONEXTRAS_API UParametricMovementComponent : public UBaseMovementComponent
{
public:

	GENERATED_BODY()

	UParametricMovementComponent();

	virtual void BeginPlay() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	void InitializeSimulationState(FParametricSyncState* SyncState, FParametricAuxState* AuxState);
	void ProduceInput(const int32 SimTimeMS, FParametricInputCmd* Cmd);
	void RestoreFrame(const FParametricSyncState* SyncState, const FParametricAuxState* AuxState);
	void FinalizeFrame(const FParametricSyncState* SyncState, const FParametricAuxState* AuxState);

	UFUNCTION(BlueprintCallable, Category="Networking")
	void EnableInterpolationMode(bool bValue);

protected:

	void InitializeNetworkPredictionProxy() override;
	TPimplPtr<FParametricMovementSimulation> OwnedParametricMovementSimulation;

	// ------------------------------------------------------------------------
	// Temp Parametric movement example
	//	The essence of this movement simulation is to map some Time value to a transform. That is it.
	//	(It could be mapped via a spline, a curve, a simple blueprint function, etc).
	//	What is below is just a simple C++ implementation to stand things up. Most likely we would 
	//	do additional subclasses to vary the way this is implemented)
	// ------------------------------------------------------------------------
	
	/** Disables starting the simulation. For development/testing ease of use */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ParametricMovement)
	bool bDisableParametricMovementSimulation = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParametricMovement, meta=(ExposeOnSpawn=true))
	FSimpleParametricMotion ParametricMotion;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ParametricMovementNetworking)
	bool bEnableDependentSimulation = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ParametricMovementNetworking)
	bool bEnableInterpolation = true;

	/** Calls ForceNetUpdate every frame. Has slightly different behavior than a very high NetUpdateFrequency */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ParametricMovementNetworking)
	bool bEnableForceNetUpdate = false;

	/** Sets NetUpdateFrequency on parent. This is editable on the component and really just meant for use during development/test maps */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ParametricMovementNetworking)
	float ParentNetUpdateFrequency = 0.f;

	TOptional<float> PendingPlayRate = 1.f;
};

