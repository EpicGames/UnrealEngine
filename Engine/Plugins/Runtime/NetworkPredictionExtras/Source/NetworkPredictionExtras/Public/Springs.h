// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/EngineTypes.h"
#include "Springs.generated.h"


struct FSpring;
static FArchive& operator<<(FArchive& Ar, FSpring& Spring);

USTRUCT(BlueprintType)
struct FSpring
{
	GENERATED_BODY()

	friend FArchive& operator<<(FArchive& Ar, FSpring& Spring);

	FSpring()
		: LocalPosition(ForceInitToZero)
		, LocalDirection(ForceInitToZero)
		, TraceChannel(ECC_Visibility)
		, Stiffness(0.0f)
		, DampingStrength(0.0f)
		, RaycastLength(0.0f)
		, NaturalLength(0.0f)
		, PreviousLength(0.0f)
	{}

	bool operator==(const FSpring& Spring) const
	{
		 return LocalPosition == Spring.LocalPosition
			&& LocalDirection == Spring.LocalDirection
			&& TraceChannel == Spring.TraceChannel
			&& Stiffness == Spring.Stiffness
			&& DampingStrength == Spring.DampingStrength
			&& RaycastLength == Spring.RaycastLength
			&& NaturalLength == Spring.NaturalLength;
			//&& PreviousLength == Spring.PreviousLength; // Deliberately not reconciled, not replicating this value.
	}

	// Position of spring relative to actor, raycast starts here and force is applied at this position.
	UPROPERTY(BlueprintReadWrite, Category="Spring")
	FVector LocalPosition;

	// Direction of spring force applied to body.
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	FVector LocalDirection;

	// Trace channel of raycast
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	TEnumAsByte<ECollisionChannel> TraceChannel;
	
	// Strength of spring force.
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	float Stiffness;

	// Strength of damping term.
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	float DampingStrength;

	// Lenght of raycast.
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	float RaycastLength;

	// Natural length of spring when it is not stretched/compressed.
	UPROPERTY(BlueprintReadWrite, Category = "Spring")
	float NaturalLength;

	float GetPreviousLength() const { return PreviousLength; }
	void SetPreviousLength(float InLength)  { PreviousLength = InLength; }
	void ResetState() { SetPreviousLength(NaturalLength); }
private:
	// Length of spring from last step.
	float PreviousLength;
};

static FArchive& operator<<(FArchive& Ar, FSpring& Spring)
{
	Ar << Spring.LocalPosition;
	Ar << Spring.LocalDirection;
	Ar << Spring.TraceChannel;
	Ar << Spring.Stiffness;
	Ar << Spring.DampingStrength;
	Ar << Spring.RaycastLength;
	Ar << Spring.NaturalLength;
	//Ar << Spring.PreviousLength; Deliberately not serialized as we do not need to replicate.
	return Ar;
}
