// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetworkPredictionExtrasLatentLoad : ModuleRules
	{
		public NetworkPredictionExtrasLatentLoad(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "NetPredExtrasLL";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"NetworkPrediction",
					"Core",
                    "CoreUObject",
                    "Engine",
                    "RenderCore",
					"InputCore",
					"PhysicsCore",
					"Chaos",
					"Engine"
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"TraceLog",
					"Engine"
                }
				);

			// Only needed for the PIE delegate in FNetworkPredictionModule::StartupModule
            if (Target.Type == TargetType.Editor) {
                PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                });
            }

		}
	}
}
