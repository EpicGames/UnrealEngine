// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionExtrasLatentLoadModule.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "NetworkPredictionSimulation.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionTrace.h"

#define LOCTEXT_NAMESPACE "FNetworkPredictionModuleLatentLoad"

DEFINE_LOG_CATEGORY_STATIC(LogNetworkPredictionExtrasLatentLoad, Display, All);

class FNetworkPredictionExtrasLatentLoadModule : public INetworkPredictionExtrasLatentLoadModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

void FNetworkPredictionExtrasLatentLoadModule::StartupModule()
{
	UE_LOGF(LogNetworkPredictionExtrasLatentLoad, Display, "STARTUP FNetworkPredictionExtrasLatentLoadModule");
}


void FNetworkPredictionExtrasLatentLoadModule::ShutdownModule()
{
	UE_LOGF(LogNetworkPredictionExtrasLatentLoad, Display, "SHUTDOWN FNetworkPredictionExtrasLatentLoadModule");
}

IMPLEMENT_MODULE( FNetworkPredictionExtrasLatentLoadModule, NetworkPredictionExtrasLatentLoad )
#undef LOCTEXT_NAMESPACE

// --------------------------------------------------------------------------------------------------

struct FLatentTestCue
{
	FLatentTestCue() = default;

	NETSIMCUE_BODY();

	int32 X=0;

	using Traits = NetSimCueTraits::Strong;
	
	void NetSerialize(FArchive& Ar)
	{
	}
	
	bool NetIdentical(const FLatentTestCue& Other) const
	{
		return true;
	}
};

NETSIMCUE_REGISTER(FLatentTestCue, TEXT("FLatentTestCue"));


struct FLatentTestCue2
{
	FLatentTestCue2() = default;

	NETSIMCUE_BODY();

	int32 X=0;
	
	using Traits = NetSimCueTraits::Strong;
	
	void NetSerialize(FArchive& Ar)
	{
	}
	
	bool NetIdentical(const FLatentTestCue2& Other) const
	{
		return true;
	}
};

NETSIMCUE_REGISTER(FLatentTestCue2, TEXT("FLatentTestCue2"));


struct FLatentTestCueSet
{
	template<typename TDispatchTable>
	static void RegisterNetSimCueTypes(TDispatchTable& DispatchTable)
	{
		DispatchTable.template RegisterType<FLatentTestCue>();
		DispatchTable.template RegisterType<FLatentTestCue2>();
	}
};

class FLatentTestCueHandler
{
public:
	void HandleCue(const FLatentTestCue& Cue, const FNetSimCueSystemParamemters& SystemParameters) { }
	void HandleCue(const FLatentTestCue2& Cue, const FNetSimCueSystemParamemters& SystemParameters) { }
};

NETSIMCUESET_REGISTER(FLatentTestCueHandler, FLatentTestCueSet);


// --------------------------------------------------------------------------------------------------

struct FLatentInputCmd
{
	float InputValue=0;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << InputValue;
	}
	
	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("InputValue: %.4f\n", InputValue);
	}
};

struct FLatentSyncState
{
	float Total=0;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Total;
	}

	// Compare this state with AuthorityState. return true if a reconcile (correction) should happen
	bool ShouldReconcile(const FLatentSyncState& AuthorityState) const
	{
		UE_NP_TRACE_RECONCILE(FMath::Abs<float>(Total - AuthorityState.Total) > SMALL_NUMBER, "Total:");
		return false;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Total: %.4f\n", Total);
	}

	void Interpolate(const FLatentSyncState* From, const FLatentSyncState* To, float PCT)
	{
		Total = FMath::Lerp(From->Total, To->Total, PCT);
	}
};

struct FLatentAuxState
{
	float Multiplier=1;
	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << Multiplier;
	}

	bool ShouldReconcile(const FLatentAuxState& Authority) const
	{
		UE_NP_TRACE_RECONCILE(FMath::Abs<float>(Multiplier - Authority.Multiplier) > SMALL_NUMBER, "Multiplier:");
		return false;
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Multiplier: %.4f\n", Multiplier);
	}

	void Interpolate(const FLatentAuxState* From, const FLatentAuxState* To, float PCT)
	{
		Multiplier = FMath::Lerp(From->Multiplier, To->Multiplier, PCT);
	}
};

using TLatentNetworkSimulationBufferTypes = TNetworkPredictionStateTypes<FLatentInputCmd, FLatentSyncState, FLatentAuxState>;

struct FNetSimTimeStep;

class FLatentNetworkSimulation
{
public:
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<TLatentNetworkSimulationBufferTypes>& Input, const TNetSimOutput<TLatentNetworkSimulationBufferTypes>& Output) { }
};

class FLatentSimDriver
{
public:
	void InitializeSimulationState(FLatentSyncState* Sync, FLatentAuxState* Aux) { }
	void ProduceInput(const int32 DeltaTimeMS, FLatentInputCmd* Cmd) { }
	void FinalizeFrame(const FLatentSyncState* Sync, const FLatentAuxState* Aux) { }
};

class FLatentNetworkModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = FLatentNetworkSimulation;
	using StateTypes = TLatentNetworkSimulationBufferTypes;
	using Driver = FLatentSimDriver;

	static const TCHAR* GetName() { return TEXT("LatentNetSim"); }
};

NP_MODEL_REGISTER(FLatentNetworkModelDef);
