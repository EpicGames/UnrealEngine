// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "NetworkPredictionExtrasLatentLoadModule.generated.h"

// The purpose of this module is simply to provide some NetworkPrediction types (ModelDefs, Cues) in a module that can be
// dynamically loaded/unloaded: in order to stress NP's "mini type systems". The types included in this module are non functional.
// you cannot actually do anything with them at runtime.

class INetworkPredictionExtrasLatentLoadModule : public IModuleInterface
{

public:
	
	static inline INetworkPredictionExtrasLatentLoadModule& Get()
	{
		return FModuleManager::LoadModuleChecked<INetworkPredictionExtrasLatentLoadModule>( "NetworkPredictionExtrasLatentLoad" );
	}
	
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "NetworkPredictionExtrasLatentLoad" );
	}
};

// Required stub object for unloading (non abandoning) module
UCLASS()
class UNetworkPredictionExtrasLatentLoadStubObject : public UObject
{
	GENERATED_BODY()

	UNetworkPredictionExtrasLatentLoadStubObject() { }
};